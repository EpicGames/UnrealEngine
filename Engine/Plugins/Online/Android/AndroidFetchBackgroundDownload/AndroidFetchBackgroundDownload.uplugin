{
	"FileVersion" : 3,
	"Version" : 1,
	"VersionName" : "1.0",
	"FriendlyName" : "Android Fetch Background Download",
	"Description" : "An Android plugin for enabling BackgroundHTTP requests to work while the app is backgrounded through use of the Fetch API.",
	"Category" : "BackgroundHTTP",
	"CreatedBy" : "Epic Games, Inc.",
	"CreatedByURL" : "https://epicgames.com",
	"EnabledByDefault" : false,
	"SupportedTargetPlatforms": [
		"Android"
	],
	"Modules": 
	[
		{
			"Name": "AndroidFetchBackgroundDownload",
			"Type": "RuntimeNoCommandlet",
			"LoadingPhase": "None",
			"PlatformAllowList": [
				"Android"
			]
		}
	],
	"Plugins": 
	[
		{
			"Name": "AndroidBackgroundService",
			"Enabled": true
		}
	],
	"LocalizationTargets": [
		{
			"Name": "AndroidFetchBackgroundDownload",
			"LoadingPolicy": "Always"
		}
	]
}