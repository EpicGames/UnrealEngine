[AndroidBackgroundHTTP.DefaultTextLoc]
;Represents PolyglotTextData structure for our Localized Notification Text Defaults. Has to be manually constructed from separate strings as we can't rely on the UObject parsing we normally would for an FPolyglotTextData object as this is loaded very early before UObjects are loaded.
;Should always put native string first and follow with all other translations

;Title Text
+DefaultNotificationText_Title=("native","Downloading")
+DefaultNotificationText_Title=("ar","جارٍ التنزيل")
+DefaultNotificationText_Title=("de","Lädt herunter")
+DefaultNotificationText_Title=("es","Descargando")
+DefaultNotificationText_Title=("es-419","Descargando")
+DefaultNotificationText_Title=("fr","Téléchargement")
+DefaultNotificationText_Title=("it","Download in corso")
+DefaultNotificationText_Title=("ja","ダウンロード中")
+DefaultNotificationText_Title=("ko","다운로드 중")
+DefaultNotificationText_Title=("pl","Pobieranie")
+DefaultNotificationText_Title=("pt-BR","Baixando")
+DefaultNotificationText_Title=("ru","Загрузка")
+DefaultNotificationText_Title=("tr","İndiriliyor")
+DefaultNotificationText_Title=("zh-CN","下载中")
+DefaultNotificationText_Title=("zh-Hant","下載中")

;Content Text
+DefaultNotificationText_Content=("native","Download in Progress {DownloadPercent}")
+DefaultNotificationText_Content=("ar","التنزيل قيد التقدم {DownloadPercent}")
+DefaultNotificationText_Content=("de","Download läuft {DownloadPercent}")
+DefaultNotificationText_Content=("es","Descarga en curso {DownloadPercent}")
+DefaultNotificationText_Content=("es-419","Descarga en curso {DownloadPercent}")
+DefaultNotificationText_Content=("fr","Téléchargé à {DownloadPercent}")
+DefaultNotificationText_Content=("it","Download in corso {DownloadPercent}")
+DefaultNotificationText_Content=("ja","ダウンロード進行中 {DownloadPercent}")
+DefaultNotificationText_Content=("ko","다운로드 진행 중 {DownloadPercent}")
+DefaultNotificationText_Content=("pl","Pobieranie w toku ({DownloadPercent})")
+DefaultNotificationText_Content=("pt-BR","Download em Andamento {DownloadPercent}")
+DefaultNotificationText_Content=("ru","Выполняется загрузка: {DownloadPercent}")
+DefaultNotificationText_Content=("tr","İndirme devam ediyor: {DownloadPercent}")
+DefaultNotificationText_Content=("zh-CN","正在下载{DownloadPercent}")
+DefaultNotificationText_Content=("zh-Hant","下載進度{DownloadPercent}")

;Completed Text
+DefaultNotificationText_Complete=("native","Download Complete")
+DefaultNotificationText_Complete=("ar","اكتمل التنزيل")
+DefaultNotificationText_Complete=("de","Download abgeschlossen")
+DefaultNotificationText_Complete=("es","Descarga completada")
+DefaultNotificationText_Complete=("es-419","Descarga completa")
+DefaultNotificationText_Complete=("fr","Téléchargement terminé")
+DefaultNotificationText_Complete=("it","Download completato")
+DefaultNotificationText_Complete=("ja","ダウンロード完了")
+DefaultNotificationText_Complete=("ko","다운로드 완료")
+DefaultNotificationText_Complete=("pl","Pobieranie zakończone")
+DefaultNotificationText_Complete=("pt-BR","Download Concluído")
+DefaultNotificationText_Complete=("ru","Загрузка завершена")
+DefaultNotificationText_Complete=("tr","İndirme Tamamlandı")
+DefaultNotificationText_Complete=("zh-CN","下载完成")
+DefaultNotificationText_Complete=("zh-Hant","下載完成")

;Cancel Button Text
+DefaultNotificationText_Cancel=("native","Cancel")
+DefaultNotificationText_Cancel=("ar","إلغاء")
+DefaultNotificationText_Cancel=("de","Abbrechen")
+DefaultNotificationText_Cancel=("es","Cancelar")
+DefaultNotificationText_Cancel=("es-419","Cancelar")
+DefaultNotificationText_Cancel=("fr","Annuler")
+DefaultNotificationText_Cancel=("it","Annulla")
+DefaultNotificationText_Cancel=("ja","キャンセル")
+DefaultNotificationText_Cancel=("ko","취소")
+DefaultNotificationText_Cancel=("pl","Anuluj")
+DefaultNotificationText_Cancel=("pt-BR","Cancelar")
+DefaultNotificationText_Cancel=("ru","Отменить")
+DefaultNotificationText_Cancel=("tr","İptal")
+DefaultNotificationText_Cancel=("zh-CN","取消")
+DefaultNotificationText_Cancel=("zh-Hant","取消")

;NoInternet Text
+DefaultNotificationText_NoInternet=("native","No Internet available")
+DefaultNotificationText_NoInternet=("ar","الإنترنت غير متوفر")
+DefaultNotificationText_NoInternet=("de","Keine Internetverbindung verfügbar")
+DefaultNotificationText_NoInternet=("es","No hay conexión a internet")
+DefaultNotificationText_NoInternet=("es-419","No hay conexión a internet")
+DefaultNotificationText_NoInternet=("fr","Internet indisponible")
+DefaultNotificationText_NoInternet=("it","Internet non disponibile")
+DefaultNotificationText_NoInternet=("ja","インターネットが利用できません")
+DefaultNotificationText_NoInternet=("ko","인터넷이 없습니다")
+DefaultNotificationText_NoInternet=("pl","Brak internetu")
+DefaultNotificationText_NoInternet=("pt-BR","Internet indisponível")
+DefaultNotificationText_NoInternet=("ru","Нет доступа в интернет")
+DefaultNotificationText_NoInternet=("tr","İnternet bağlantısı yok")

;Airplane Mode text
+DefaultNotificationText_AirplaneMode=("native","Airplane Mode Enabled. Download paused")
+DefaultNotificationText_AirplaneMode=("ar","تم تفعيل وضع الطائرة. تم إيقاف التنزيل مؤقتًا")
+DefaultNotificationText_AirplaneMode=("de","Flugzeugmodus aktiviert. Download pausiert")
+DefaultNotificationText_AirplaneMode=("es","Modo avión habilitado. Descarga pausada.")
+DefaultNotificationText_AirplaneMode=("es-419","Modo Avión habilitado. Descarga pausada")
+DefaultNotificationText_AirplaneMode=("fr","Mode avion activé. Téléchargement en pause")
+DefaultNotificationText_AirplaneMode=("it","Modalità aereo abilitata. Download in pausa")
+DefaultNotificationText_AirplaneMode=("ja","機内モードが有効にされました。ダウンロード一時停止中")
+DefaultNotificationText_AirplaneMode=("ko","비행기 모드가 활성화되었습니다. 다운로드 일시정지됨")
+DefaultNotificationText_AirplaneMode=("pl","Włączono tryb samolotowy. Pobieranie wstrzymane")
+DefaultNotificationText_AirplaneMode=("pt-BR","Modo Avião Habilitado. Download pausado")
+DefaultNotificationText_AirplaneMode=("ru","Авиарежим включён. Загрузка приостановлена")
+DefaultNotificationText_AirplaneMode=("tr","Uçak Modu Etkin. İndirme durduruldu")
+DefaultNotificationText_AirplaneMode=("zh-CN","飞机模式已启用。下载已暂停")
+DefaultNotificationText_AirplaneMode=("zh-Hant","已啟用飛行模式。 下載已暫停")

;Data Saver Enabled Text
+DefaultNotificationText_DataSaverEnabled=("native","Data Saver Enabled. Download paused")
+DefaultNotificationText_DataSaverEnabled=("ar","تم تفعيل حافظ البيانات. تم إيقاف التنزيل مؤقتًا")
+DefaultNotificationText_DataSaverEnabled=("de","Flugzeugmodus aktiviert. Download pausiert")
+DefaultNotificationText_DataSaverEnabled=("es","Ahorro de datos habilitado. Descarga pausada.")
+DefaultNotificationText_DataSaverEnabled=("es-419","Almacenamiento de datos habilitado. Descarga pausada")
+DefaultNotificationText_DataSaverEnabled=("fr","Mode économiseur de données activé. Téléchargement en pause")
+DefaultNotificationText_DataSaverEnabled=("it","Risparmio dati abilitato. Download in pausa")
+DefaultNotificationText_DataSaverEnabled=("ja","データセーバーが有効にされました。ダウンロード一時停止中")
+DefaultNotificationText_DataSaverEnabled=("ko","데이터 절약 모드가 활성화되었습니다. 다운로드 일시정지됨")
+DefaultNotificationText_DataSaverEnabled=("pl","Włączono oszczędzanie danych. Pobieranie wstrzymane")
+DefaultNotificationText_DataSaverEnabled=("pt-BR","Proteção de Dados Habilitada. Download pausado")
+DefaultNotificationText_DataSaverEnabled=("ru","Сохранение данных включено. Загрузка приостановлена")
+DefaultNotificationText_DataSaverEnabled=("tr","Veri Tasarrufu Etkin. İndirme durduruldu")
+DefaultNotificationText_DataSaverEnabled=("zh-CN","流量节省已启用。下载已暂停")
+DefaultNotificationText_DataSaverEnabled=("zh-Hant","已啟用數據儲存。 下載已暫停")

;Waiting for cellular text
+DefaultNotificationText_WaitingForCellular=("native","Waiting for cellular approval")
+DefaultNotificationText_WaitingForCellular=("ar","في انتظار السماح باستخدام بيانات الهاتف")
+DefaultNotificationText_WaitingForCellular=("de","Warten auf Bestätigung der mobilen Datenverwendung")
+DefaultNotificationText_WaitingForCellular=("es","En espera de aprobación del uso de datos móviles")
+DefaultNotificationText_WaitingForCellular=("es-419","Esperando aprobación de uso de datos móviles")
+DefaultNotificationText_WaitingForCellular=("fr","En attente d'autorisation de l'utilisation des données mobiles")
+DefaultNotificationText_WaitingForCellular=("it","In attesa di autorizzazione per l’utilizzo dei dati mobili")
+DefaultNotificationText_WaitingForCellular=("ja","モバイルデータ通信の許可を待っています")
+DefaultNotificationText_WaitingForCellular=("ko","모바일 데이터 사용 승인 기다리는 중")
+DefaultNotificationText_WaitingForCellular=("pl","Oczekiwanie na zatwierdzenie użycia danych komórkowych")
+DefaultNotificationText_WaitingForCellular=("pt-BR","Aguardando a autorização de uso de dados móveis")
+DefaultNotificationText_WaitingForCellular=("ru","Ожидание разрешения на использование сотовой сети для передачи данных")
+DefaultNotificationText_WaitingForCellular=("tr","Hücresel veri onayı bekleniyor")

;Approve Text
+DefaultNotificationText_Approve=("native","Approve")
+DefaultNotificationText_Approve=("ar","السماح")
+DefaultNotificationText_Approve=("de","Bestätigen")
+DefaultNotificationText_Approve=("es","Aprobar")
+DefaultNotificationText_Approve=("es-419","Aprobar")
+DefaultNotificationText_Approve=("fr","Autoriser")
+DefaultNotificationText_Approve=("it","Autorizza")
+DefaultNotificationText_Approve=("ja","許可する")
+DefaultNotificationText_Approve=("ko","승인")
+DefaultNotificationText_Approve=("pl","Zatwierdź")
+DefaultNotificationText_Approve=("pt-BR","Autorizar")
+DefaultNotificationText_Approve=("ru","Разрешить")
+DefaultNotificationText_Approve=("tr","Onayla")
