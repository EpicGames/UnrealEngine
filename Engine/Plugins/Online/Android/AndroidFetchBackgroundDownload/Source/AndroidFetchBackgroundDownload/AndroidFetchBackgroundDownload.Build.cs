// Copyright Epic Games, Inc. All Rights Reserved.

using System.IO;

namespace UnrealBuildTool.Rules
{
	public class AndroidFetchBackgroundDownload : ModuleRules
	{
		public AndroidFetchBackgroundDownload(ReadOnlyTargetRules Target) : base(Target)
		{
			ShortName = "AndroidFBGDL";

			PrivateDependencyModuleNames.AddRange(new string[]
				{
					"Core",
					"CoreUObject",
					"Engine",
					"Launch",
					"AndroidBackgroundService",
					"BackgroundHTTP",
					"BuildSettings",
				});

			if (Target.Platform == UnrealTargetPlatform.Android)
			{
				string PluginPath = Utils.MakePathRelativeTo(ModuleDirectory, Target.RelativeEnginePath);
				AdditionalPropertiesForReceipt.Add("AndroidPlugin", Path.Combine(PluginPath, "AndroidFetchBackgroundDownload_UPL.xml"));
			}
		}
	}
}
