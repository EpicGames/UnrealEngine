<?xml version="1.0" encoding="utf-8"?>
<!-- Google Cloud Messaging plugin additions-->
<root xmlns:android="http://schemas.android.com/apk/res/android">
	<!-- init section is always evaluated once per architecture -->
	<init>
		<log text="AndroidFetchBackgroundDownload init"/>
	</init>
	
	<buildGradleAdditions>
		<insert>
//AndroidFetchBackgroundDownload -- START
	repositories {
		mavenCentral()
		maven { url 'https://jitpack.io' }
	}
	dependencies {
		//Fetch2 library used to manage downloads at java level
	    api "org.jetbrains.kotlin:kotlin-stdlib:2.0.20"
		api "androidx.room:room-runtime:2.7.1"
		api "androidx.core:core-ktx:1.13.1"
		api "com.squareup.okhttp3:okhttp:4.12.0"
		api "com.squareup.okhttp3:okhttp-urlconnection:4.12.0"
		</insert>		
		<insertValue value="implementation files ('$S(AbsPluginDir)/ThirdParty/fetch2-release.aar')"/>
		<insertNewline/>
		<insertValue value="implementation files ('$S(AbsPluginDir)/ThirdParty/fetch2core-release.aar')"/>
		<insertNewline/>
		<insertValue value="implementation files ('$S(AbsPluginDir)/ThirdParty/fetch2fileserver-release.aar')"/>
		<insertNewline/>
		<insertValue value="implementation files ('$S(AbsPluginDir)/ThirdParty/fetch2okhttp-release.aar')"/>
		<insertNewline/>
		<insertValue value="implementation files ('$S(AbsPluginDir)/ThirdParty/fetch2rx-release.aar')"/>
		<insertNewline/>
		<insertValue value="implementation files ('$S(AbsPluginDir)/ThirdParty/fetchmigrator-release.aar')"/>
		<insertNewline/>
		<insert>
		
		//Used for parsing from JSON files in DownloadDescription.java
		//excluding hamcrest to avoid collision with junit that also includes hamcrest
		implementation ('com.googlecode.json-simple:json-simple:1.1.1') {
			exclude group: 'org.hamcrest', module: 'hamcrest-core'
		}
	}
//AndroidFetchBackgroundDownload -- STOP
		</insert>
	</buildGradleAdditions>

	<androidManifestUpdates>
		<addElements tag="application">
			<receiver android:name="com.epicgames.unreal.download.DownloadManager$CancelReceiver" android:exported="false"/>
		</addElements>
	</androidManifestUpdates>
	
	<!-- optional additions to proguard -->
	<proguardAdditions>
		<insert>
			# AndroidFetchBackgroundDownload UPL Additions -- START

			# Android Background HTTP Classes with native calls ...

			-keep class com.epicgames.unreal.download.DownloadManager {
			public *;
			}

			-keep class com.epicgames.unreal.download.UEDownloadWorker {
			public *;
			}

			-keep class com.epicgames.unreal.download.datastructs.DownloadDescription {
			public *;
			}

			# AndroidFetchBackgroundDownload UPL Additions -- STOP
		</insert>
	</proguardAdditions>
	
	<!-- optional files or directories to copy or delete from Intermediate/Android/APK before ndk-build -->
	<prebuildCopies>
		<log text="Copying AndroidFetchBackgroundDownload files to staging"/>
		<copyDir src="$S(PluginDir)/Java" dst="$S(BuildDir)/src/com/epicgames/unreal" />
	</prebuildCopies>

	<!-- set build version to BranchName-Changelist, see C++ equivalent in AndroidPlatformBackgroundHttpRequest.cpp-->
	<gameActivityClassAdditions>
		<insertValue value="public static final String BGDL_BUILD_VERSION = &quot;$S(EngineBranchName)&quot;;"/>
		<insertNewline/>
	</gameActivityClassAdditions>

	<gameActivityOnResumeAdditions>
		<insert>
			com.epicgames.unreal.download.DownloadManager.Resume();
		</insert>
	</gameActivityOnResumeAdditions>
</root>
