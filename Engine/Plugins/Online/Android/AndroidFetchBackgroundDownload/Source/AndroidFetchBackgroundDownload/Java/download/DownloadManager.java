// Copyright Epic Games, Inc. All Rights Reserved.
package com.epicgames.unreal.download;

import android.annotation.SuppressLint;
import android.app.ForegroundServiceStartNotAllowedException;
import android.app.Notification;
import android.app.NotificationChannel;
import android.app.NotificationManager;
import android.app.PendingIntent;
import android.content.BroadcastReceiver;
import android.content.Context;
import android.content.Intent;
import android.content.IntentFilter;
import android.content.SharedPreferences;
import android.net.ConnectivityManager;
import android.os.Build;
import android.os.Handler;
import android.os.Looper;
import android.provider.Settings;

import androidx.annotation.Keep;
import androidx.annotation.MainThread;
import androidx.annotation.NonNull;
import androidx.concurrent.futures.CallbackToFutureAdapter;
import androidx.core.app.NotificationCompat;
import androidx.core.net.ConnectivityManagerCompat;
import androidx.work.BackoffPolicy;
import androidx.work.ExistingWorkPolicy;
import androidx.work.ForegroundInfo;
import androidx.work.ListenableWorker;
import androidx.work.OneTimeWorkRequest;
import androidx.work.Operation;
import androidx.work.WorkManager;
import androidx.work.WorkerParameters;

import com.epicgames.unreal.CellularReceiver;
import com.epicgames.unreal.GameActivity;
import com.epicgames.unreal.GameApplication;
import com.epicgames.unreal.Logger;
import com.epicgames.unreal.download.datastructs.DownloadDescription;
import com.epicgames.unreal.download.datastructs.DownloadNotificationDescription;
import com.epicgames.unreal.download.datastructs.DownloadQueueDescription;
import com.epicgames.unreal.download.datastructs.DownloadWorkerParameterKeys;
import com.epicgames.unreal.download.fetch.FetchManagerInterfaceHelper;
import com.epicgames.unreal.download.fetch.FetchManagerNew;
import com.epicgames.unreal.network.NetworkChangedManager;
import com.epicgames.unreal.network.NetworkConnectivityClient;
import com.google.common.util.concurrent.ListenableFuture;

import java.io.IOException;
import java.nio.charset.StandardCharsets;
import java.nio.file.FileVisitResult;
import java.nio.file.Files;
import java.nio.file.Path;
import java.nio.file.Paths;
import java.nio.file.SimpleFileVisitor;
import java.nio.file.attribute.BasicFileAttributes;
import java.text.NumberFormat;
import java.util.ArrayList;
import java.util.Comparator;
import java.util.HashMap;
import java.util.List;
import java.util.Map;
import java.util.Objects;
import java.util.concurrent.ConcurrentHashMap;
import java.util.concurrent.ExecutionException;
import java.util.concurrent.TimeUnit;
import java.util.concurrent.atomic.AtomicInteger;
import java.util.stream.Collectors;
import java.util.stream.Stream;

import dalvik.annotation.optimization.FastNative;

@Keep
public class DownloadManager
{
	static final @NonNull Logger Log = new Logger("UE", DownloadManager.class.getSimpleName());
	
	static final @NonNull Handler Handler = new Handler(Looper.getMainLooper());

	final static String WorkName = "BackgroundHttpDownload";
	final static int InitialBackoffDelayInSeconds = 10;

	static SharedPreferences BackgroundPreferences;
	
	@SuppressLint("StaticFieldLeak")
	static Task Task;
	
	@Keep
	@FastNative
	native static void OnProgress(@NonNull String DestinationLocation, long BytesWrittenSinceLastCall, long TotalBytesWritten);
	@Keep
	@FastNative
	native static void OnComplete(@NonNull String DestinationLocation, boolean bDidSucceed, long TotalBytesDownloaded, long DownloadDuration, long DownloadStartTimeUTC, long DownloadEndTimeUTC);
	@Keep
	@FastNative
	native static void OnStart(@NonNull String DestinationLocation, @NonNull String DebugString, long StartTimeUTC);
	
	static boolean IsInForeground()
	{
		return BackgroundPreferences != null;
	}

	static @NonNull SharedPreferences GetBackgroundPreferences(@NonNull Context context)
	{
		return context.getSharedPreferences("BackgroundPreferences", Context.MODE_PRIVATE);
	}
	
	static @NonNull String GetTempDownloadRootPath(@NonNull SharedPreferences backgroundPreferences)
	{
		String tempDownloadRootPath = backgroundPreferences.getString("TempDownloadRootPath", "");
		assert !tempDownloadRootPath.isEmpty();
		return tempDownloadRootPath;
	}
	
	static ListenableFuture<Operation.State.SUCCESS> EnqueueWorker(@NonNull Context context)
	{
		assert IsInForeground();
		
		return WorkManager.getInstance(context).enqueueUniqueWork(WorkName, ExistingWorkPolicy.KEEP, new OneTimeWorkRequest.Builder(Worker.class)
			.setBackoffCriteria(BackoffPolicy.EXPONENTIAL, InitialBackoffDelayInSeconds, TimeUnit.SECONDS)
			.build()).getResult();
	}

	static void CancelWorker(@NonNull Context context)
	{
		WorkManager.getInstance(context).cancelUniqueWork(WorkName);
	}
	
	@Keep
	public static class CancelReceiver extends BroadcastReceiver
	{
		@MainThread
		@Override
		public void onReceive(@NonNull Context context, Intent intent)
		{
			if (intent != null)
			{
				if (Task != null && Task.Worker != null)
				{
					if (!IsInForeground() || GameActivity.IsActivityPaused())
					{
						Task.Worker.Cancel();
					}
				}
			}
		}
	}

	public static class RestrictBackgroundStatusReceiver extends BroadcastReceiver
	{
		@NonNull ConnectivityManager ConnectivityManager;
		
		public RestrictBackgroundStatusReceiver(@NonNull Context context)
		{
			assert Task != null;
			
			ConnectivityManager = context.getSystemService(ConnectivityManager.class);
			
			Task.RestrictBackgroundStatus = ConnectivityManager.getRestrictBackgroundStatus();
		}
		
		@MainThread
		@Override
		public void onReceive(@NonNull Context context, Intent intent)
		{
			if (intent != null)
			{
				if (Task != null)
				{
					Task.SetRestrictBackgroundStatus(ConnectivityManager.getRestrictBackgroundStatus());
				}
			}
		}
	}

	static class Task implements Runnable, DownloadProgressListener, NetworkConnectivityClient.Listener, SharedPreferences.OnSharedPreferenceChangeListener
	{
		final @NonNull Context Context;
		final @NonNull Path DownloadDescriptionsPath;

		final @NonNull ConcurrentHashMap<String, DownloadDescription> DownloadDescriptions = new ConcurrentHashMap<>();
		final @NonNull AtomicInteger RequestCount = new AtomicInteger();

		FetchManagerNew FetchManager;
		SharedPreferences CellularNetworkPreferences;
		RestrictBackgroundStatusReceiver RestrictBackgroundStatusReceiver;
		
		boolean bIsPaused;
		boolean bIsNetworkLost;
		NetworkConnectivityClient.NetworkTransportType NetworkTransportType = NetworkConnectivityClient.NetworkTransportType.UNKNOWN;
		boolean bIsNetworkMetered;
		@ConnectivityManagerCompat.RestrictBackgroundStatus int RestrictBackgroundStatus;

		ListenableFuture<Operation.State.SUCCESS> EnqueueWorkerFuture;
		Worker Worker;

		public Task(@NonNull Context context, @NonNull SharedPreferences backgroundPreferences)
		{
			Context = context;
			DownloadDescriptionsPath = Paths.get(GetTempDownloadRootPath(backgroundPreferences), "DownloadDescriptions");
		}
		
		@MainThread
		void Initialize()
		{
			assert Task == null;
			Task = this;

			FetchManager = (FetchManagerNew)FetchManagerInterfaceHelper.GetSharedManager();
			CellularNetworkPreferences = Context.getSharedPreferences("CellularNetworkPreferences", android.content.Context.MODE_PRIVATE);
			RestrictBackgroundStatusReceiver = new RestrictBackgroundStatusReceiver(Context);
			
			NetworkChangedManager.getInstance().addListener(this, true);

			CellularNetworkPreferences.registerOnSharedPreferenceChangeListener(this);

			Context.registerReceiver(RestrictBackgroundStatusReceiver, new IntentFilter(ConnectivityManager.ACTION_RESTRICT_BACKGROUND_CHANGED));
		}

		@MainThread
		public void Start()
		{
			Log.debug("Starting");
			
			assert Task == null;
			assert IsInForeground();
			
			try
			{
				Files.createDirectories(DownloadDescriptionsPath);
			}
			catch (IOException e)
			{
				e.printStackTrace();
			}
			
			Initialize();
		}
		
		@MainThread
		public void Restart(@NonNull SharedPreferences backgroundPreferences)
		{
			Log.debug("Restarting");
			
			assert Task == null;
			assert !IsInForeground();

			ArrayList<DownloadDescription> downloadDescriptions;
			try (Stream<Path> files = Files.list(DownloadDescriptionsPath))
			{
				downloadDescriptions = files.map(file ->
				{
					try
					{
						return DownloadDescription.FromJSON(new String(Files.readAllBytes(file), StandardCharsets.UTF_8));
					}
					catch (IOException ignored)
					{
						return null;
					}
				}).filter(Objects::nonNull).collect(Collectors.toCollection(ArrayList::new));
			}
			catch (IOException ignored)
			{
				return;
			}

			if (downloadDescriptions.isEmpty())
			{
				return;
			}

			Initialize();

			DownloadQueueDescription downloadQueueDescription = new DownloadQueueDescription(downloadDescriptions, backgroundPreferences);
			downloadQueueDescription.ProgressListener = this;
			FetchManager.Enqueue(Context, downloadQueueDescription);
		}

		@MainThread
		public boolean IsWaitingForCellularApproval()
		{
			return NetworkTransportType == NetworkConnectivityClient.NetworkTransportType.CELLULAR &&
				!(RestrictBackgroundStatus == ConnectivityManagerCompat.RESTRICT_BACKGROUND_STATUS_WHITELISTED || CellularNetworkPreferences.getInt("AllowCellular", 0) > 0);
		}

		@MainThread
		public boolean IsResumable()
		{
			return !bIsNetworkLost && !bIsPaused && (IsInForeground() || !IsWaitingForCellularApproval());
		}
		
		@MainThread
		public void SetRestrictBackgroundStatus(@ConnectivityManagerCompat.RestrictBackgroundStatus int restrictBackgroundStatus)
		{
			if (RestrictBackgroundStatus != restrictBackgroundStatus)
			{
				RestrictBackgroundStatus = restrictBackgroundStatus;
				
				if (!IsInForeground())
				{
					if (Task.IsResumable())
					{
						Log.debug("SetRestrictBackgroundStatus, resuming");
						FetchManager.ResumeAllDownloads();
					}
					else
					{
						Log.debug("SetRestrictBackgroundStatus, pausing");
						FetchManager.PauseAllDownloads();
					}
				}
			}
		}

		@MainThread
		public void Enqueue(@NonNull DownloadDescription downloadDescription, @NonNull SharedPreferences backgroundPreferences)
		{
			assert IsInForeground();

			DownloadQueueDescription downloadQueueDescription = new DownloadQueueDescription(new ArrayList<>(List.of(downloadDescription)), backgroundPreferences);
			downloadQueueDescription.ProgressListener = this;
			FetchManager.Enqueue(Context, downloadQueueDescription);

			if (Worker == null && (EnqueueWorkerFuture == null || EnqueueWorkerFuture.isDone()))
			{
				EnqueueWorkerFuture = EnqueueWorker(Context);
			}
		}

		@MainThread
		public void Pause()
		{
			assert Task == this;
			assert Worker == null;

			if (GameActivity.IsActivityPaused())
			{
				if (!bIsPaused)
				{
					if (IsResumable())
					{
						Log.debug("Pause, pausing");
						FetchManager.PauseAllDownloads();
					}

					bIsPaused = true;
				}
			}
			else
			{
				EnqueueWorkerFuture = EnqueueWorker(Context);
			}
		}

		@MainThread
		public void Resume()
		{
			assert Task == this;
			assert Worker == null;

			if (bIsPaused)
			{
				bIsPaused = false;

				if (IsResumable())
				{
					Log.debug("Resume, resuming");
					FetchManager.ResumeAllDownloads();
				}

				if (EnqueueWorkerFuture == null || EnqueueWorkerFuture.isDone())
				{
					EnqueueWorkerFuture = EnqueueWorker(Task.Context);
				}
			}
		}

		@MainThread
		void Dispose()
		{
			assert Task == this;
			Task = null;
			
			Context.unregisterReceiver(RestrictBackgroundStatusReceiver);

			CellularNetworkPreferences.unregisterOnSharedPreferenceChangeListener(this);

			NetworkChangedManager.getInstance().removeListener(this);
		}

		@MainThread
		void Stop()
		{
			Log.debug("Stopping");
			
			Dispose();

			FetchManager.StopWork(null);
			
			Handler.removeCallbacks(this);
		}

		@MainThread
		void AllComplete()
		{
			Dispose();
			
			final boolean didAllSucceed = DownloadDescriptions.isEmpty();
			
			if (didAllSucceed)
			{
				Context.getSharedPreferences("UEDownloadWorker", android.content.Context.MODE_PRIVATE).edit().putString("AllDownloadsCompletedTime", NumberFormat.getInstance().format(System.currentTimeMillis() / 1000.0f)).commit();
				
				if (!IsInForeground())
				{
					CellularNetworkPreferences.edit().putInt("AllowCellular", 0).commit();
				}
			}

			if (Worker != null)
			{
				Worker.AllComplete(didAllSucceed ? ListenableWorker.Result.success() : ListenableWorker.Result.retry());
			}
			else
			{
				CancelWorker(Context);
			}
		}

		@MainThread
		public void CancelAll()
		{
			assert Task == this;

			Stop();

			for (DownloadDescription downloadDescription : DownloadDescriptions.values())
			{
				try
				{
					Files.delete(DownloadDescriptionsPath.resolve(Paths.get(downloadDescription.DestinationLocation).getFileName()));
				}
				catch (IOException ignored)
				{
				}
			}

			if (Worker != null)
			{
				Worker.AllComplete(ListenableWorker.Result.success());
			}
			else
			{
				CancelWorker(Context);
			}
		}
		
		boolean Removed(@NonNull DownloadDescription downloadDescription)
		{
			try
			{
				Files.delete(DownloadDescriptionsPath.resolve(Paths.get(downloadDescription.DestinationLocation).getFileName()));
			}
			catch (IOException ignored)
			{
			}

			int requestCount = RequestCount.decrementAndGet();
			assert requestCount >= 0;
			return requestCount == 0;
		}

		@MainThread
		public void Pause(@NonNull String destinationLocation)
		{
			assert Task == this;

			Log.debug("@*@ PauseRequest: " + destinationLocation);
			FetchManager.PauseDownload(destinationLocation, true);
		}

		@MainThread
		public void Resume(@NonNull String destinationLocation)
		{
			assert Task == this;

			Log.debug("@*@ ResumeRequest: " + destinationLocation);
			FetchManager.PauseDownload(destinationLocation, false);
		}
		
		@MainThread
		public void Cancel(@NonNull String destinationLocation)
		{
			assert Task == this;

			FetchManager.CancelDownload(destinationLocation);

			DownloadDescription downloadDescription = DownloadDescriptions.remove(destinationLocation);
			if (downloadDescription != null && Removed(downloadDescription))
			{
				AllComplete();
			}
		}

		@MainThread
		@Override
		public void run()
		{
			if (Task != null)
			{
				assert Task == this;

				if (RequestCount.get() == 0)
				{
					AllComplete();
				}
			}
		}

		@Override
		public void OnDownloadStart(@NonNull String DestinationLocation, @NonNull String DebugString, long StartTimeUTC)
		{
			if (IsInForeground())
			{
				OnStart(DestinationLocation, DebugString, StartTimeUTC);
			}
		}

		@Override
		public void OnDownloadProgress(@NonNull String DestinationLocation, long TotalBytesWritten, long BytesWrittenSinceLastCall)
		{
			if (IsInForeground())
			{
				OnProgress(DestinationLocation, TotalBytesWritten, BytesWrittenSinceLastCall);
			}
		}

		@MainThread
		@Override
		public void OnDownloadGroupProgress(int GroupID, final int Progress, final boolean Indeterminate)
		{
			if (Worker != null)
			{
				Worker.UpdateProgress(Indeterminate ? -1 : Progress);
			}
		}

		@Override
		public void OnDownloadComplete(@NonNull String DestinationLocation, UEDownloadWorker.EDownloadCompleteReason CompleteReason)
		{
			assert Thread.holdsLock(FetchManager.SyncRoot());
			
			final boolean bDidSucceed = CompleteReason == UEDownloadWorker.EDownloadCompleteReason.Success;
			if (IsInForeground() || bDidSucceed)
			{
				DownloadDescription downloadDescription = DownloadDescriptions.remove(DestinationLocation);
				if (downloadDescription != null)
				{
					if (Removed(downloadDescription))
					{
						Handler.post(this);
					}

					if (IsInForeground())
					{
						OnComplete(DestinationLocation, bDidSucceed, downloadDescription.TotalDownloadedBytes, downloadDescription.DownloadDuration, downloadDescription.DownloadStartTimeUTC, downloadDescription.DownloadEndTimeUTC);
					}
				}
			}
		}

		@Override
		public void OnDownloadMetrics(@NonNull String DestinationLocation, long TotalBytesDownloaded, long DownloadDuration, long DownloadStartTimeUTC, long DownloadEndTimeUTC)
		{
			DownloadDescription downloadDescription = DownloadDescriptions.get(DestinationLocation);
			if (downloadDescription != null)
			{
				downloadDescription.TotalDownloadedBytes = TotalBytesDownloaded;
				downloadDescription.DownloadDuration = DownloadDuration;
				downloadDescription.DownloadStartTimeUTC = DownloadStartTimeUTC;
				downloadDescription.DownloadEndTimeUTC = DownloadEndTimeUTC;
			}
		}

		@Override
		public void OnAllDownloadsComplete(boolean bDidAllRequestsSucceed)
		{
		}

		@MainThread
		@Override
		public void OnDownloadEnqueued(@NonNull DownloadDescription DownloadDescription)
		{
			assert Thread.holdsLock(FetchManager.SyncRoot());
			
			if (DownloadDescriptions.put(DownloadDescription.DestinationLocation, DownloadDescription) == null)
			{
				RequestCount.incrementAndGet();
			}
				
			if (IsInForeground())
			{
				try
				{
					Files.write(DownloadDescriptionsPath.resolve(Paths.get(DownloadDescription.DestinationLocation).getFileName()), DownloadDescription.ToJSON().getBytes(StandardCharsets.UTF_8));
				}
				catch (IOException ignored)
				{
				}
			}
		}
		
		@Override
		public void onNetworkAvailable(NetworkConnectivityClient.NetworkTransportType networkTransportType, boolean isNetworkMetered)
		{
			Handler.post(() ->
			{
				if (Task != null)
				{
					assert Task == this;

					NetworkTransportType = networkTransportType;
					bIsNetworkMetered = isNetworkMetered;
					
					if (bIsNetworkLost)
					{
						bIsNetworkLost = false;

						if (IsResumable())
						{
							Log.debug("onNetworkAvailable, resuming");
							FetchManager.ResumeAllDownloads();
						}
					}
				}
			});
		}

		@Override
		public void onNetworkLost()
		{
			Handler.post(() ->
			{
				if (Task != null)
				{
					assert Task == this;
					
					if (!bIsNetworkLost)
					{
						if (IsResumable())
						{
							Log.debug("onNetworkLost, pausing");
							FetchManager.PauseAllDownloads();
						}

						bIsNetworkLost = true;
					}
				}
			});
		}

		@MainThread
		@Override
		public void onSharedPreferenceChanged(@NonNull SharedPreferences sharedPreferences, String key)
		{
			if (IsResumable())
			{
				Log.debug("onSharedPreferenceChanged, resuming");
				FetchManager.ResumeAllDownloads();
			}
			else
			{
				Log.debug("onSharedPreferenceChanged, pausing");
				FetchManager.PauseAllDownloads();
			}
		}
	}

	@Keep
	public static class Worker extends ListenableWorker implements CallbackToFutureAdapter.Resolver<ListenableWorker.Result>, Runnable
	{
		CallbackToFutureAdapter.Completer<ListenableWorker.Result> Completer;

		long StartTimeMs;
		static final long MaxRuntimeMs = TimeUnit.HOURS.toMillis(5) + TimeUnit.MINUTES.toMillis(30); // 5.5 hrs, before Android's 6hr foreground service limit

		DownloadNotificationDescription NotificationDescription;
		
		PendingIntent PendingCancelIntent;
		PendingIntent PendingApproveIntent;
		PendingIntent PendingNotificationIntent;
		
		public Worker(@NonNull Context context, @NonNull WorkerParameters params)
		{
			super(context, params);
		}
		
		void Initialize(@NonNull Context context, @NonNull SharedPreferences backgroundPreferences)
		{
			NotificationDescription = new DownloadNotificationDescription(backgroundPreferences, context, Log);

			PendingCancelIntent = PendingIntent.getBroadcast(context, 0, new Intent(context, CancelReceiver.class), PendingIntent.FLAG_IMMUTABLE);
			PendingApproveIntent = PendingIntent.getBroadcast(context, 0, new Intent(context, CellularReceiver.class), PendingIntent.FLAG_IMMUTABLE);
			PendingNotificationIntent = PendingIntent.getActivity(context, NotificationDescription.NotificationID, new Intent(context, GameActivity.class)
				.setFlags(Intent.FLAG_ACTIVITY_CLEAR_TOP | Intent.FLAG_ACTIVITY_SINGLE_TOP)
				.putExtra("localNotificationID", NotificationDescription.NotificationID)
				.putExtra("localNotificationAppLaunched", true), PendingIntent.FLAG_IMMUTABLE);
			
			NotificationManager notificationManager = context.getSystemService(NotificationManager.class);
			if (notificationManager.getNotificationChannel(NotificationDescription.NotificationChannelID) == null)
			{
				String channelName = NotificationDescription.NotificationChannelName;
				if (channelName.equals("ue-downloadworker-channel"))
				{
					// check for an override for default channel name
					channelName = com.epicgames.unreal.LocalNotificationReceiver.getLocalizedResource(context, "UEBackgroundChannel", channelName);
				}
				notificationManager.createNotificationChannel(new NotificationChannel(NotificationDescription.NotificationChannelID, channelName, NotificationDescription.NotificationChannelImportance));
			}
		}

		@NonNull Notification CreateAirplaneModeNotification(@NonNull Context context)
		{
			final boolean indeterminate = !NotificationDescription.ShouldShowPercentage || NotificationDescription.Indeterminate;
			String notificationText;
			if (NotificationDescription.CurrentProgress < NotificationDescription.MAX_PROGRESS)
			{
				notificationText = "";
			}
			else
			{
				notificationText = NotificationDescription.ContentCompleteText;
			}

			NotificationCompat.Builder builder = new NotificationCompat.Builder(context, NotificationDescription.NotificationChannelID)
				.setContentTitle(NotificationDescription.AirplaneModeText)
				.setTicker(NotificationDescription.TitleText)
				.setContentText(notificationText)
				.setContentIntent(PendingNotificationIntent)
				.setProgress(NotificationDescription.MAX_PROGRESS, NotificationDescription.CurrentProgress, indeterminate)
				.setOngoing(true)
				.setOnlyAlertOnce(true)
				.setSmallIcon(NotificationDescription.SmallIconResourceID)
				.setSilent(true);
			if (!IsInForeground())
			{
				builder.addAction(NotificationDescription.CancelIconResourceID, NotificationDescription.CancelText, PendingCancelIntent);
			}
			return builder.build();
		}

		@NonNull Notification CreateDataSaverEnabledNotification(@NonNull Context context)
		{
			final boolean indeterminate = !NotificationDescription.ShouldShowPercentage || NotificationDescription.Indeterminate;
			String notificationText;
			if (NotificationDescription.CurrentProgress < NotificationDescription.MAX_PROGRESS)
			{
				notificationText = "";
			}
			else
			{
				notificationText = NotificationDescription.ContentCompleteText;
			}

			NotificationCompat.Builder builder = new NotificationCompat.Builder(context, NotificationDescription.NotificationChannelID)
				.setContentTitle(NotificationDescription.DataSaverEnabledText)
				.setTicker(NotificationDescription.TitleText)
				.setContentText(notificationText)
				.setContentIntent(PendingNotificationIntent)
				.setProgress(NotificationDescription.MAX_PROGRESS, NotificationDescription.CurrentProgress, indeterminate)
				.setOngoing(true)
				.setOnlyAlertOnce(true)
				.setSmallIcon(NotificationDescription.SmallIconResourceID)
				.setSilent(true);
			if (!IsInForeground())
			{
				builder.addAction(NotificationDescription.CancelIconResourceID, NotificationDescription.CancelText, PendingCancelIntent);
			}
			return builder.build();
		}

		@NonNull Notification CreateNoInternetDownloadNotification(@NonNull Context context)
		{
			final boolean indeterminate = !NotificationDescription.ShouldShowPercentage || NotificationDescription.Indeterminate;
			String notificationText;
			if (NotificationDescription.CurrentProgress < NotificationDescription.MAX_PROGRESS)
			{
				notificationText = "";
			}
			else
			{
				notificationText = NotificationDescription.ContentCompleteText;
			}

			NotificationCompat.Builder builder = new NotificationCompat.Builder(context, NotificationDescription.NotificationChannelID)
				.setContentTitle(NotificationDescription.NoInternetAvailable)
				.setTicker(NotificationDescription.TitleText)
				.setContentText(notificationText)
				.setContentIntent(PendingNotificationIntent)
				.setProgress(NotificationDescription.MAX_PROGRESS, NotificationDescription.CurrentProgress, indeterminate)
				.setOngoing(true)
				.setOnlyAlertOnce(true)
				.setSmallIcon(NotificationDescription.SmallIconResourceID)
				.setSilent(true);
			if (!IsInForeground())
			{
				builder.addAction(NotificationDescription.CancelIconResourceID, NotificationDescription.CancelText, PendingCancelIntent);
			}
			return builder.build();
		}

		@NonNull Notification CreateCellularWaitNotification(@NonNull Context context)
		{
			final boolean indeterminate = !NotificationDescription.ShouldShowPercentage || NotificationDescription.Indeterminate;
			String notificationText;
			if (NotificationDescription.CurrentProgress < NotificationDescription.MAX_PROGRESS)
			{
				notificationText = "";
			}
			else
			{
				notificationText = NotificationDescription.ContentCompleteText;
			}

			NotificationCompat.Builder builder = new NotificationCompat.Builder(context, NotificationDescription.NotificationChannelID)
				.setContentTitle(NotificationDescription.WaitingForCellularText)
				.setTicker(NotificationDescription.WaitingForCellularText)
				.setContentText(notificationText)
				.setContentIntent(PendingNotificationIntent)
				.setProgress(NotificationDescription.MAX_PROGRESS, NotificationDescription.CurrentProgress, indeterminate)
				.setOngoing(true)
				.setOnlyAlertOnce(true)
				.setSmallIcon(NotificationDescription.SmallIconResourceID)
				.setSilent(true);
			if (!IsInForeground())
			{
				builder.addAction(NotificationDescription.CancelIconResourceID, NotificationDescription.ApproveText, PendingApproveIntent);
				builder.addAction(NotificationDescription.CancelIconResourceID, NotificationDescription.CancelText, PendingCancelIntent);
			}
			return builder.build();
		}
		
		@NonNull Notification CreateDownloadProgressNotification(@NonNull Context context)
		{
			final boolean indeterminate = !NotificationDescription.ShouldShowPercentage || NotificationDescription.Indeterminate;
			String notificationText;
			if (NotificationDescription.CurrentProgress < NotificationDescription.MAX_PROGRESS)
			{
				if (indeterminate)
				{
					notificationText = NotificationDescription.ContentText.replace("%3d%%", "");
				}
				else
				{
					notificationText = String.format(NotificationDescription.ContentText, NotificationDescription.CurrentProgress);
				}
			}
			else
			{
				notificationText = NotificationDescription.ContentCompleteText;
			}

			NotificationCompat.Builder builder = new NotificationCompat.Builder(context, NotificationDescription.NotificationChannelID)
				.setContentTitle(NotificationDescription.TitleText)
				.setTicker(NotificationDescription.TitleText)
				.setContentText(notificationText)
				.setContentIntent(PendingNotificationIntent)
				.setProgress(NotificationDescription.MAX_PROGRESS, NotificationDescription.CurrentProgress, indeterminate)
				.setOngoing(true)
				.setOnlyAlertOnce(true)
				.setSmallIcon(NotificationDescription.SmallIconResourceID)
				.setSilent(true);
			if (!IsInForeground() || GameActivity.IsActivityPaused())
			{
				builder.addAction(NotificationDescription.CancelIconResourceID, NotificationDescription.CancelText, PendingCancelIntent);
			}
			return builder.build();
		}

		@MainThread
		@NonNull Notification CreateNotification(@NonNull Context context)
		{
			assert Task != null;
			
			//Task.FetchManager.RequestGroupProgressUpdate(0, Task);

			if (Task.bIsNetworkLost && Settings.Global.getInt(context.getContentResolver(), Settings.Global.AIRPLANE_MODE_ON, 0) != 0)
			{
				return CreateAirplaneModeNotification(context);
			}
			else if (!IsInForeground() && Task.bIsNetworkMetered && Task.RestrictBackgroundStatus == ConnectivityManagerCompat.RESTRICT_BACKGROUND_STATUS_ENABLED)
			{
				return CreateDataSaverEnabledNotification(context);
			}
			else if (Task.bIsNetworkLost)
			{
				return CreateNoInternetDownloadNotification(context);
			}
			else if (Task.IsWaitingForCellularApproval())
			{
				return CreateCellularWaitNotification(context);
			}
			else
			{
				return CreateDownloadProgressNotification(context);
			}
		}
		
		@MainThread
		public void UpdateProgress(int value)
		{
			assert NotificationDescription != null;
			
			NotificationDescription.CurrentProgress = value;
			NotificationDescription.Indeterminate = value < 0;
		}
		
		public void AllComplete(Result result)
		{
			Handler.removeCallbacks(this);
			
			Completer.set(result);
		}

		@MainThread
		public void Cancel()
		{
			Log.debug("Cancelling");
			
			if (Completer.setCancelled())
			{
				Cancelled();
			}
		}

		@MainThread
		void Cancelled()
		{
			Handler.removeCallbacks(this);

			if (Task != null && Task.Worker == this)
			{
				Task.Worker = null;

				if (!IsInForeground())
				{
					Task.Stop();
				}
				else
				{
					Task.Pause();
				}
			}
		}
		
		@MainThread
		@Override
		public @NonNull ListenableFuture<Result> startWork()
		{
			StartTimeMs = System.currentTimeMillis();

			ListenableFuture<ListenableWorker.Result> future = CallbackToFutureAdapter.getFuture(this);

			final Context context = getApplicationContext();
			
			SharedPreferences backgroundPreferences;
			if (!IsInForeground())
			{
				backgroundPreferences = GetBackgroundPreferences(context);
				
				FetchManagerInterfaceHelper.InitSharedManager(context);

				try
				{
					new Task(context, backgroundPreferences).Restart(backgroundPreferences);
				}
				catch (Throwable t)
				{
					t.printStackTrace();
					
					Completer.setException(t);
					
					if (Task != null)
					{
						Task.Stop();
					}
					
					return future;
				}
			}
			else
			{
				backgroundPreferences = BackgroundPreferences;
			}

			if (Task != null)
			{
				Initialize(context, backgroundPreferences);
				
				Task.Worker = this;
				Task.EnqueueWorkerFuture = null;

				run();
			}
			else
			{
				Completer.set(Result.success());
			}
			
			return future;
		}

		@Override
		public void onStopped()
		{
			Log.debug("onStopped");
			
			Handler.post(this::Cancelled);
		}
		
		@Override
		public Object attachCompleter(@NonNull CallbackToFutureAdapter.Completer<ListenableWorker.Result> completer)
		{
			assert Completer == null;
			Completer = completer;
			return null;
		}

		@MainThread
		@Override
		public void run()
		{
			if (Task != null && Task.Worker == this)
			{
				if (System.currentTimeMillis() - StartTimeMs > MaxRuntimeMs)
				{
					Log.error("Worker exceeded maximum runtime of 5.5 hours. Forcing retry to avoid foreground service timeout.");
					Task.AllComplete();
					return;
				}

				@SuppressLint("InlinedApi")
				ListenableFuture<Void> future = setForegroundAsync(new ForegroundInfo(NotificationDescription.NotificationID, CreateNotification(getApplicationContext()), android.content.pm.ServiceInfo.FOREGROUND_SERVICE_TYPE_DATA_SYNC));
				future.addListener(() ->
				{
					try
					{
						future.get();
						
						Handler.postDelayed(this, 500);
					}
					catch (Throwable t)
					{
						if (Build.VERSION.SDK_INT >= Build.VERSION_CODES.S && t instanceof ExecutionException e && e.getCause() instanceof ForegroundServiceStartNotAllowedException)
						{
							Log.debug("Not allowed to start as a foreground service");
						}
						else
						{
							t.printStackTrace();
							
							if (Completer.setException(t))
							{
								onStopped();
							}
						}
					}
				}, Runnable::run);
			}
		}
	}

	@Keep
	public static void Initialize(@NonNull HashMap<String, ?> params, boolean waitTillDone)
	{
		Log.debug("Initializing");
		
		Handler.post(() ->
		{
			assert BackgroundPreferences == null;

			final Context context = GameApplication.getAppContext();
			
			if (Task != null)
			{
				Task.CancelAll();
			}
			else
			{
				CancelWorker(context);
			}
			
			assert Task == null;

			synchronized (params)
			{
				BackgroundPreferences = GetBackgroundPreferences(context);
				params.notifyAll();
			}

			try (Stream<Path> files = Files.list(Paths.get(GetTempDownloadRootPath(BackgroundPreferences))))
			{
				files.filter(file -> !file.endsWith("URLMap") && !file.endsWith(GameActivity.BGDL_BUILD_VERSION)).forEach(file ->
				{
					Log.debug("cleaning up: " + file.getFileName());

					try
					{
						Files.walkFileTree(file, new SimpleFileVisitor<>()
						{
							@Override
							public FileVisitResult visitFile(Path file, BasicFileAttributes attrs)
							{
								try
								{
									Files.delete(file);
								}
								catch (IOException ignored)
								{
								}
								
								return FileVisitResult.CONTINUE;
							}

							@Override
							public FileVisitResult postVisitDirectory(Path dir, IOException exc)
							{
								try
								{
									Files.delete(dir);
								}
								catch (IOException ignored)
								{
								}
								
								return FileVisitResult.CONTINUE;
							}
						});
					}
					catch (IOException ignored)
					{
					}
				});
			}
			catch (IOException ignored)
			{
			}

			SharedPreferences.Editor editor = BackgroundPreferences.edit();
			for (Map.Entry<String, ?> entry : params.entrySet())
			{
				if (entry.getValue() instanceof String value && !value.equals(BackgroundPreferences.getString(entry.getKey(), "")))
				{
					editor.putString(entry.getKey(), value);
				}
				else if (entry.getValue() instanceof Boolean value && value != BackgroundPreferences.getBoolean(entry.getKey(), false))
				{
					editor.putBoolean(entry.getKey(), value);
				}
				else if (entry.getValue() instanceof Integer value && value != BackgroundPreferences.getInt(entry.getKey(), 0))
				{
					editor.putInt(entry.getKey(), value);
				}
			}
			editor.commit();
		});
		
		if (waitTillDone)
		{
			// Not a fan of this. Remove once AsyncDownloader is fixed, or using OkHttp directly
			synchronized (params)
			{
				try
				{
					while (BackgroundPreferences == null)
					{
						params.wait(250);
					}
				}
				catch (InterruptedException ignored)
				{
				}
			}
		}
	}

	@Keep
	public static void SetMaxConcurrentDownloads(int maxConcurrentDownloads)
	{
		BackgroundPreferences.edit().putInt(DownloadWorkerParameterKeys.DOWNLOAD_MAX_CONCURRENT_REQUESTS_KEY, maxConcurrentDownloads).apply();
	}

	@Keep
	public static void Enqueue(@NonNull DownloadDescription downloadDescription)
	{
		Handler.post(() ->
		{
			if (Task == null)
			{
				new Task(GameApplication.getAppContext(), BackgroundPreferences).Start();
			}

			Task.Enqueue(downloadDescription, BackgroundPreferences);
		});
	}

	@Keep
	@MainThread
	public static void Resume()
	{
		if (Task != null && Task.Worker == null)
		{
			Task.Resume();
		}
	}

	@Keep
	public static void Pause(@NonNull String destinationLocation)
	{
		Handler.post(() ->
		{
			if (Task != null)
			{
				Task.Pause(destinationLocation);
			}
		});
	}
	
	@Keep
	public static void Resume(@NonNull String destinationLocation)
	{
		Handler.post(() ->
		{
			if (Task != null)
			{
				Task.Resume(destinationLocation);
			}
		});
	}
	
	@Keep
	public static void Cancel(@NonNull String destinationLocation)
	{
		Handler.post(() ->
		{
			if (Task != null)
			{
				Task.Cancel(destinationLocation);
			}
		});
	}

	@Keep
	public static String GetActiveDestinations()
	{
		if (Task != null && Task.FetchManager != null)
		{
			String[] Destinations = Task.FetchManager.GetRunningDestinationLocations();
			return String.join("\n", Destinations);
		}
		return "";
	}
}