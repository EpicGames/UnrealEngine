// Copyright Epic Games, Inc. All Rights Reserved.
package com.epicgames.unreal.download;

import androidx.annotation.NonNull;

import com.epicgames.unreal.download.UEDownloadWorker.EDownloadCompleteReason;
import com.epicgames.unreal.download.datastructs.DownloadDescription;

//Interface for a class that wants to be notified when a download completes
public interface DownloadProgressListener
{
	void OnDownloadStart(@NonNull String DestinationLocation, @NonNull String DebugString, long StartTimeUTC);
	void OnDownloadProgress(@NonNull String DestinationLocation, long TotalBytesWritten, long BytesWrittenSinceLastCall);
	void OnDownloadGroupProgress(int GroupID, int Progress, boolean Indeterminate);
	void OnDownloadComplete(@NonNull String DestinationLocation, EDownloadCompleteReason CompleteReason);
	void OnDownloadMetrics(@NonNull String DestinationLocation, long TotalBytesDownloaded, long DownloadDuration, long DownloadStartTimeUTC, long DownloadEndTimeUTC);
	void OnAllDownloadsComplete(boolean bDidAllRequestsSucceed);
	void OnDownloadEnqueued(@NonNull DownloadDescription DownloadDescription);
}