// Copyright Epic Games, Inc. All Rights Reserved.
package com.epicgames.unreal.download;

import android.content.Intent;

import android.app.Notification;
import android.app.NotificationChannel;
import android.app.NotificationManager;
import android.app.PendingIntent;
import android.content.Context;
import android.net.Uri;
import android.os.Build;
import android.content.SharedPreferences;

import androidx.annotation.Nullable;
import androidx.annotation.NonNull;

import androidx.core.app.NotificationCompat;
import androidx.work.Data;
import androidx.work.ForegroundInfo;
import androidx.work.WorkManager;
import androidx.work.WorkerParameters;

import java.io.File;

import com.epicgames.unreal.GameActivity;

import com.epicgames.unreal.Logger;
import com.epicgames.unreal.download.datastructs.DownloadDescription;
import com.epicgames.unreal.workmanager.UEWorker;

import com.epicgames.unreal.download.datastructs.DownloadNotificationDescription;
import com.epicgames.unreal.download.DownloadProgressListener;
import com.epicgames.unreal.download.datastructs.DownloadQueueDescription;
import com.epicgames.unreal.download.datastructs.DownloadWorkerParameterKeys;
import com.epicgames.unreal.download.fetch.FetchManagerInterface;
import com.epicgames.unreal.download.fetch.FetchManagerInterfaceHelper;
import com.epicgames.unreal.LocalNotificationReceiver;
import com.epicgames.unreal.CellularReceiver;

import com.epicgames.unreal.network.NetworkConnectivityClient;
import com.epicgames.unreal.network.NetworkChangedManager;

import java.util.Locale;
import java.util.concurrent.TimeUnit;

import android.app.Activity;
import android.net.ConnectivityManager;
import android.content.Context;
import android.provider.Settings;

import static android.content.Context.NOTIFICATION_SERVICE;

//Helper class to manage our different work requests and callbacks
public class UEDownloadWorker extends UEWorker implements DownloadProgressListener
{	
	public enum EDownloadCompleteReason
	{
		Success,
		Error,
		OutOfRetries
	}

	public enum MeteredType
	{
		UNMETERED,
		METERED_DATASAVER_ON,
		METERED_DATASAVER_ALLOWED,
		METERED_DATASAVER_OFF
	}
	
	public UEDownloadWorker(Context context, WorkerParameters params)
	{
		super(context,params);
		
		//Overwrite the default log to have a more specific log identifier tag
		Log = new Logger("UE", "UEDownloadWorker");
	}

	private boolean checkAirplaneMode()
	{
		Context context = getApplicationContext();
		return Settings.Global.getInt(context.getContentResolver(), Settings.Global.AIRPLANE_MODE_ON, 0) != 0;
	}

	private MeteredType checkMetered()
	{
		if (null == connectivityManager)
		{
			Context context = getApplicationContext();
			connectivityManager = (ConnectivityManager) context.getSystemService(Context.CONNECTIVITY_SERVICE);
		}

		if (connectivityManager.isActiveNetworkMetered())
		{
			switch (connectivityManager.getRestrictBackgroundStatus())
			{
				case android.net.ConnectivityManager.RESTRICT_BACKGROUND_STATUS_ENABLED:
					// background data usage blocked for this app.
					return MeteredType.METERED_DATASAVER_ON;

				case android.net.ConnectivityManager.RESTRICT_BACKGROUND_STATUS_WHITELISTED:
					// app allowed to bypass Data Saver, try to minimize Data
					return MeteredType.METERED_DATASAVER_ALLOWED;

				case android.net.ConnectivityManager.RESTRICT_BACKGROUND_STATUS_DISABLED:
					// Data Saver is disabled.
					return MeteredType.METERED_DATASAVER_OFF;
			}
		}
		return MeteredType.UNMETERED;
	}

	private boolean checkCellularAllowed()
	{
		if (null == preferences)
		{
			Context context = getApplicationContext();
			preferences = context.getSharedPreferences("CellularNetworkPreferences", context.MODE_PRIVATE);
		}
		boolean allowCell = (preferences.getInt("AllowCellular", 0) > 0);

		// treat metered data saver allowed as permission
		allowCell |= (meteredType == MeteredType.METERED_DATASAVER_ALLOWED);

		return allowCell;
	}
		
	@Override
	public void InitWorker()
	{
		super.InitWorker();
		
		if (null == mFetchManager)
		{
			FetchManagerInterfaceHelper.InitSharedManager(getApplicationContext());
			mFetchManager = FetchManagerInterfaceHelper.GetSharedManager();	
		}
		
		//Make sure we have a CancelIntent we can use to cancel this job (passed into notifications, etc)
		if (null == CancelIntent) 
		{
			CancelIntent = WorkManager.getInstance(getApplicationContext())
				.createCancelPendingIntent(getId());
		}
		if (null == ApproveIntent)
		{
			ApproveIntent = new Intent(getApplicationContext(), CellularReceiver.class);
		}
		
		//Generate our NotificationDescription so that we load important data from our InputData() to control notification content
		if (null == NotificationDescription)
		{
			NotificationDescription = new DownloadNotificationDescription(getInputData(), getApplicationContext(), Log);
		}

		if (null == NetworkListener)
		{
			NetworkListener = new NetworkConnectivityClient.Listener() {
				@Override
				public void onNetworkAvailable(NetworkConnectivityClient.NetworkTransportType networkTransportType, boolean isNetworkMetered) {
					bLostNetwork = false;
					if (mFetchManager != null) {
						mFetchManager.NetworkAvailable();
					}
				}

				@Override
				public void onNetworkLost() {
					bLostNetwork = true;
					if (mFetchManager != null) {
						mFetchManager.NetworkLost();
					}
				}
			};
			NetworkChangedManager.getInstance().addListener(NetworkListener);
		}


		bHasEnqueueHappened = false;
		bForceStopped = false;
	}
	
	@Override
	public void OnWorkerStart(String WorkID)
	{
		Log.debug("OnWorkerStart Beginning for " + WorkID);

		//TODO: TRoss this should be based on some WorkerParameter and handled in UEWorker
		//Set this as an important task so that it continues even when the app closes, etc.
		//Do this immediately as we only have limited time to call this after worker start
		setForegroundAsync(CreateForegroundInfo(NotificationDescription));

		super.OnWorkerStart(WorkID);
		
		if (mFetchManager == null)
		{
			Log.error("OnWorkerStart called without a valid FetchInstance! Failing Worker and completing!");
			SetWorkResult_Failure();
			return;
		}
		
		//Setup downloads in mFetchManager
		if (null == preferencesBackground)
		{
			Context context = getApplicationContext();
			preferencesBackground = context.getSharedPreferences("BackgroundDownloadTasks", context.MODE_PRIVATE);
		}
		CurrentTaskFilename = preferencesBackground.getString("TaskFilename", "");
		
		QueueDescription = new DownloadQueueDescription(CurrentTaskFilename, getInputData(), getApplicationContext(), Log);
		QueueDescription.ProgressListener = this;

       //Have to have parsed some DownloadDescriptions to have any meaningful work to do
		if ((QueueDescription == null) || (QueueDescription.DownloadDescriptions.size() == 0))
		{
			Log.error("Invalid QueueDescription! No DownloadDescription list for queued UEDownloadWorker! Worker InputData:" + getInputData());
			SetWorkResult_Failure();
			return;
		}

		//Kick off our enqueue request with the FetchManager
		mFetchManager.EnqueueRequests(getApplicationContext(),QueueDescription);

		//Enter actual loop until work is finished
		Log.verbose("Entering OnWorkerStart Loop");
		final long startTimeMs = System.currentTimeMillis();
		final long maxRuntimeMs = TimeUnit.HOURS.toMillis(5) + TimeUnit.MINUTES.toMillis(30); // 5.5 hrs, before Android's 6hr foreground service limit
		try
		{
			while (bReceivedResult == false && !isStopped())
			{
				if (System.currentTimeMillis() - startTimeMs > maxRuntimeMs)
				{
					Log.error("OnWorkerStart exceeded maximum runtime of 5.5 hours. Forcing retry to avoid foreground service timeout.");
					SetWorkResult_Retry();
					break;
				}

				// check if taskfile has been changed
				String RequestedTaskFilename = preferencesBackground.getString("TaskFilename", CurrentTaskFilename);
				if (RequestedTaskFilename.equals(CurrentTaskFilename))
				{
					Tick(QueueDescription);
					Thread.sleep(500);
				}
				else
				{
					String OldFilename = CurrentTaskFilename;
					CurrentTaskFilename = RequestedTaskFilename;

					// switch to new tasklist
					Log.debug("OnWorkerStart " + WorkID + " switching to " + RequestedTaskFilename);
					CleanUp(WorkID);

					QueueDescription = new DownloadQueueDescription(CurrentTaskFilename, getInputData(), getApplicationContext(), Log);
					QueueDescription.ProgressListener = this;

					//Have to have parsed some DownloadDescriptions to have any meaningful work to do
					if ((QueueDescription == null) || (QueueDescription.DownloadDescriptions.size() == 0))
					{
						Log.error("Invalid QueueDescription! No DownloadDescription list for queued UEDownloadWorker! Worker InputData:" + getInputData());
						SetWorkResult_Failure();
						return;
					}

					//Kick off our enqueue request with the FetchManager
					bHasEnqueueHappened = false;
					mFetchManager.EnqueueRequests(getApplicationContext(),QueueDescription);
					
					//Finally clean up the old task file
					File DeleteFile = new File(OldFilename);
					if (DeleteFile.exists())
					{
						DeleteFile.delete();
						Log.debug("Deleted DownloadDescriptorJSONFile " + OldFilename + " in CleanUp");
					}
				}
			}
		} 
		catch (InterruptedException e) 
		{
			Log.error("Exception trying to sleep thread. Setting work result to retry and shutting down");
			e.printStackTrace();
			
			SetWorkResult_Retry();
		}
		finally
		{
			CleanUp(WorkID);

			Log.debug("Finishing OnWorkerStart. CachedResult:" + CachedResult + " bReceivedResult:" + bReceivedResult);
		}
	}

	// Checking what the current network type is.
	// Will pause all downloads if the network type is Cellular without permission
	private void NetworkTypeCheck()
	{
		meteredType = checkMetered();

		// check if airplane mode is enabled (may be cause of lost network)
		bAirplaneMode = checkAirplaneMode();

		if (bGameThreadIsActive)
		{
			// check status but don't try to affect pause/resume state as game will handle it
			if (bLostNetwork)
			{
				bWaitingForCellularApproval = false;
				return;
			}

			// if we are on cellular we need permission
			NetworkConnectivityClient.NetworkTransportType networkType = NetworkChangedManager.getInstance().networkTransportTypeCheck();
			if (networkType == NetworkConnectivityClient.NetworkTransportType.CELLULAR)
			{
				// check if we have permission granted
				if (!checkCellularAllowed())
				{
					bWaitingForCellularApproval = true;
					return;
				}

			}

			// working network and not waiting for cellular
			bWaitingForCellularApproval = false;
		}
		else
		{
			// if we lose network connectivity or data saver mode enabled when inactive, pause all downloads
			if (bLostNetwork || (meteredType == MeteredType.METERED_DATASAVER_ON && !bGameThreadIsActive))
			{
				// Make sure all downloads are paused (low cost if already paused)
				mFetchManager.PauseAllDownloads();

				// not waiting for cell if no active network allowing download
				bWaitingForCellularApproval = false;

				return;
			}

			// if we are on cellular we need permission
			NetworkConnectivityClient.NetworkTransportType networkType = NetworkChangedManager.getInstance().networkTransportTypeCheck();
			if (networkType == NetworkConnectivityClient.NetworkTransportType.CELLULAR)
			{
				// check if we have permission granted
				if (!checkCellularAllowed())
				{
					// we need to pause and wait for approval (low cost if already paused)
					mFetchManager.PauseAllDownloads();
					bWaitingForCellularApproval = true;

					return;
				}
			}

			// working network and not blocked so make sure downloads running (low cost if already running)
			mFetchManager.ResumeAllDownloads();
			bWaitingForCellularApproval = false;
		}
	}
	
	private void Tick(DownloadQueueDescription QueueDescription)
	{
		//Skip any tick logic if we have already gotten a result or our download is finished as that means we are just pending our worker stopping
		//Also want to ensure enough time has passed that we have sent off our Enqueues to the FetchManager
		if (!bReceivedResult && bHasEnqueueHappened && !bForceStopped)
		{
			mFetchManager.RequestGroupProgressUpdate(QueueDescription.DownloadGroupID,  this);
			NetworkTypeCheck();
			//Keeping the code path to insert a watch dog later on
			//mFetchManager.RequestCheckDownloadsStillActive(this);

			// avoid spamming the log with endless linker errors
			if (bGameThreadIsActive)
			{
				try
				{				
					nativeAndroidBackgroundDownloadOnTick();
				}
				catch (UnsatisfiedLinkError e)
				{
				}
			}
		}
	}
	
	@Override
	public void OnWorkerStopped(String WorkID)
	{	
		bForceStopped = true;

		Log.debug("OnWorkerStopped called for " + WorkID);
		super.OnWorkerStopped(WorkID);
		
		CleanUp(WorkID);

		Log.debug("OnWorkerStopped Ending for " + WorkID + " CachedResult:" + CachedResult + " bReceivedResult:" + bReceivedResult);
	}
	
	public void CleanUp(String WorkID)
	{
		//Call stop work to make sure Fetch stops doing work while 
		if (mFetchManager != null)
		{
			mFetchManager.StopWork(WorkID);
		}
		
		//Clean up our DownloadDescriptionList file if our work is not going to re-run ever
		if (ShouldCleanupDownloadDescriptorJSONFile())
		{
			Data data = getInputData();
			if (null != data)
			{
//				String DownloadDescriptionListString = DownloadQueueDescription.GetDownloadDescriptionListFileName(data, Log);
				String DownloadDescriptionListString = CurrentTaskFilename;
				if (null != DownloadDescriptionListString)
				{
					File DeleteFile = new File(DownloadDescriptionListString);
					if (DeleteFile.exists())
					{
						DeleteFile.delete();
						Log.debug("Deleted DownloadDescriptorJSONFile " + DownloadDescriptionListString + " in CleanUp");
					}
				}
			}
		}
	}
	
	public void UpdateNotification(int CurrentProgress, boolean Indeterminate)
	{
		if (null != NotificationDescription)
		{
			NotificationDescription.CurrentProgress = CurrentProgress;
			NotificationDescription.Indeterminate = Indeterminate;
			setForegroundAsync(CreateForegroundInfo(NotificationDescription));
			
		}
		else
		{
			Log.error("Unexpected NULL NotificationDescripton during UpdateNotification!");
		}
	}

	@NonNull
	private ForegroundInfo CreateForegroundInfo(DownloadNotificationDescription Description) 
	{		
		Context context = getApplicationContext();
		NotificationManager notificationManager = GetNotificationManager(context);
		
		CreateNotificationChannel(context, notificationManager, Description);

		//Setup Opening UE app if clicked
		PendingIntent pendingNotificationIntent = null;
		{
			Intent notificationIntent = new Intent(context, GameActivity.class);
			
			// launch if closed but resume if running
			notificationIntent.setFlags(Intent.FLAG_ACTIVITY_CLEAR_TOP | Intent.FLAG_ACTIVITY_SINGLE_TOP);

			notificationIntent.putExtra("localNotificationID" , Description.NotificationID);
			notificationIntent.putExtra("localNotificationAppLaunched" , true);

			pendingNotificationIntent = PendingIntent.getActivity(context, Description.NotificationID, notificationIntent, PendingIntent.FLAG_IMMUTABLE);
		}
		
		Notification notification = null;
		if (bAirplaneMode && bLostNetwork)
		{
			notification = CreateAirplaneModeNotification(context, Description, pendingNotificationIntent);
		}
		else if (meteredType == MeteredType.METERED_DATASAVER_ON && !bGameThreadIsActive)
		{
			notification = CreateDataSaverEnabledNotification(context, Description, pendingNotificationIntent);
		}
		else if (bLostNetwork)
		{
			notification = CreateNoInternetDownloadNotification(context, Description, pendingNotificationIntent);
		}
		else if (bWaitingForCellularApproval && !checkCellularAllowed())
		{
			notification = CreateCellularWaitNotification(context, Description, pendingNotificationIntent);
		}
		else
		{
			notification = CreateDownloadProgressNotification(context, Description, pendingNotificationIntent);
		}

		return new ForegroundInfo(Description.NotificationID, notification, android.content.pm.ServiceInfo.FOREGROUND_SERVICE_TYPE_DATA_SYNC);
	}

	public Notification CreateNoInternetDownloadNotification(Context context,
		DownloadNotificationDescription Description,
		PendingIntent pendingNotificationIntent)
	{
		//Setup Notification Text Values (ContentText and ContentInfo)
		boolean bShowPercentage = ShouldShowPercentage(context);
		boolean bIsComplete = (Description.CurrentProgress >= Description.MAX_PROGRESS);
		String NotificationTextToUse = null;
		if (!bIsComplete)
		{
			if (bShowPercentage)
			{
				NotificationTextToUse = String.format(Description.ContentText, Description.CurrentProgress);
			}
			else
			{
				NotificationTextToUse = Description.ContentText.replace("%3d%%", "");
			}

			// for now don't show "Download in Progress"
			NotificationTextToUse = "";
		}
		else
		{
			NotificationTextToUse = Description.ContentCompleteText;
		}

		int CurrentProgress = bShowPercentage ? Description.CurrentProgress : 0;
		boolean Indeterminate = bShowPercentage ? Description.Indeterminate : true;

		NotificationCompat.Builder builder = new NotificationCompat.Builder(context, Description.NotificationChannelID)
			.setContentTitle(Description.NoInternetAvailable)
			.setTicker(Description.TitleText)
			.setContentText(NotificationTextToUse)
			.setContentIntent(pendingNotificationIntent)
			.setProgress(Description.MAX_PROGRESS, CurrentProgress, Indeterminate)
			.setOngoing(true)
			.setOnlyAlertOnce (true)
			.setSmallIcon(Description.SmallIconResourceID)
			.setNotificationSilent();
		if (!bGameThreadIsActive)
		{
			builder.addAction(Description.CancelIconResourceID, Description.CancelText, CancelIntent);
		}
		return builder.build();
	}

	public Notification CreateAirplaneModeNotification(Context context,
		DownloadNotificationDescription Description,
		PendingIntent pendingNotificationIntent)
	{
		//Setup Notification Text Values (ContentText and ContentInfo)
		boolean bShowPercentage = ShouldShowPercentage(context);
		boolean bIsComplete = (Description.CurrentProgress >= Description.MAX_PROGRESS);
		String NotificationTextToUse = null;
		if (!bIsComplete)
		{
			if (bShowPercentage)
			{
				NotificationTextToUse = String.format(Description.ContentText, Description.CurrentProgress);
			}
			else
			{
				NotificationTextToUse = Description.ContentText.replace("%3d%%", "");
			}

			// for now don't show "Download in Progress"
			NotificationTextToUse = "";
		}
		else
		{
			NotificationTextToUse = Description.ContentCompleteText;
		}

		int CurrentProgress = bShowPercentage ? Description.CurrentProgress : 0;
		boolean Indeterminate = bShowPercentage ? Description.Indeterminate : true;

		NotificationCompat.Builder builder = new NotificationCompat.Builder(context, Description.NotificationChannelID)
			.setContentTitle(Description.AirplaneModeText)
			.setTicker(Description.TitleText)
			.setContentText(NotificationTextToUse)
			.setContentIntent(pendingNotificationIntent)
			.setProgress(Description.MAX_PROGRESS, CurrentProgress, Indeterminate)
			.setOngoing(true)
			.setOnlyAlertOnce (true)
			.setSmallIcon(Description.SmallIconResourceID)
			.setNotificationSilent();
		if (!bGameThreadIsActive)
		{
			builder.addAction(Description.CancelIconResourceID, Description.CancelText, CancelIntent);
		}
		return builder.build();
	}

	public Notification CreateDataSaverEnabledNotification(Context context,
		DownloadNotificationDescription Description,
		PendingIntent pendingNotificationIntent)
	{
		//Setup Notification Text Values (ContentText and ContentInfo)
		boolean bShowPercentage = ShouldShowPercentage(context);
		boolean bIsComplete = (Description.CurrentProgress >= Description.MAX_PROGRESS);
		String NotificationTextToUse = null;
		if (!bIsComplete)
		{
			if (bShowPercentage)
			{
				NotificationTextToUse = String.format(Description.ContentText, Description.CurrentProgress);
			}
			else
			{
				NotificationTextToUse = Description.ContentText.replace("%3d%%", "");
			}

			// for now don't show "Download in Progress"
			NotificationTextToUse = "";
		}
		else
		{
			NotificationTextToUse = Description.ContentCompleteText;
		}

		int CurrentProgress = bShowPercentage ? Description.CurrentProgress : 0;
		boolean Indeterminate = bShowPercentage ? Description.Indeterminate : true;

		NotificationCompat.Builder builder = new NotificationCompat.Builder(context, Description.NotificationChannelID)
			.setContentTitle(Description.DataSaverEnabledText)
			.setTicker(Description.TitleText)
			.setContentText(NotificationTextToUse)
			.setContentIntent(pendingNotificationIntent)
			.setProgress(Description.MAX_PROGRESS, CurrentProgress, Indeterminate)
			.setOngoing(true)
			.setOnlyAlertOnce (true)
			.setSmallIcon(Description.SmallIconResourceID)
			.setNotificationSilent();
		if (!bGameThreadIsActive)
		{
			builder.addAction(Description.CancelIconResourceID, Description.CancelText, CancelIntent);
		}
		return builder.build();
	}

	public Notification CreateCellularWaitNotification(Context context,
		DownloadNotificationDescription Description,
		PendingIntent pendingNotificationIntent)
	{
		//Setup Notification Text Values (ContentText and ContentInfo)
		boolean bShowPercentage = ShouldShowPercentage(context);
		boolean bIsComplete = (Description.CurrentProgress >= Description.MAX_PROGRESS);
		String NotificationTextToUse = null;
		if (!bIsComplete)
		{
			if (bShowPercentage)
			{
				NotificationTextToUse = String.format(Description.ContentText, Description.CurrentProgress);
			}
			else
			{
				NotificationTextToUse = Description.ContentText.replace("%3d%%", "");
			}
			
			// for now don't show "Download in Progress"
			NotificationTextToUse = "";
		}
		else
		{
			NotificationTextToUse = Description.ContentCompleteText;
		}

		int CurrentProgress = bShowPercentage ? Description.CurrentProgress : 0;
		boolean Indeterminate = bShowPercentage ? Description.Indeterminate : true;

		NotificationCompat.Builder builder = new NotificationCompat.Builder(context, Description.NotificationChannelID)
			.setContentTitle(Description.WaitingForCellularText)
			.setTicker(Description.WaitingForCellularText)
			.setContentText(NotificationTextToUse)
			.setContentIntent(pendingNotificationIntent)
			.setProgress(Description.MAX_PROGRESS, CurrentProgress, Indeterminate)
			.setOngoing(true)
			.setOnlyAlertOnce (true)
			.setSmallIcon(Description.SmallIconResourceID)
			.setNotificationSilent();
		if (!bGameThreadIsActive)
		{
			PendingIntent cellularNotificationIntent = null;
			cellularNotificationIntent = PendingIntent.getBroadcast(context, Description.NotificationID, ApproveIntent, PendingIntent.FLAG_IMMUTABLE);
			builder.addAction(Description.CancelIconResourceID, Description.ApproveText, cellularNotificationIntent);
			builder.addAction(Description.CancelIconResourceID, Description.CancelText, CancelIntent);
		}
		return builder.build();
	}

	public Notification CreateDownloadProgressNotification(Context context,
		DownloadNotificationDescription Description,
		PendingIntent pendingNotificationIntent)
	{
		//Setup Notification Text Values (ContentText and ContentInfo)
		boolean bShowPercentage = ShouldShowPercentage(context);
		boolean bIsComplete = (Description.CurrentProgress >= Description.MAX_PROGRESS);
		String NotificationTextToUse = null;
		if (!bIsComplete)
		{
			if (bShowPercentage)
			{
				NotificationTextToUse = String.format(Description.ContentText, Description.CurrentProgress);
			}
			else
			{
				NotificationTextToUse = Description.ContentText.replace("%3d%%", "");
			}
		}
		else
		{
			NotificationTextToUse = Description.ContentCompleteText;
		}

		int CurrentProgress = bShowPercentage ? Description.CurrentProgress : 0;
		boolean Indeterminate = bShowPercentage ? Description.Indeterminate : true;

		Notification notification = new NotificationCompat.Builder(context, Description.NotificationChannelID)
			.setContentTitle(Description.TitleText)
			.setTicker(Description.TitleText)
			.setContentText(NotificationTextToUse)
			.setContentIntent(pendingNotificationIntent)
			.setProgress(Description.MAX_PROGRESS, CurrentProgress, Indeterminate)
			.setOngoing(true)
			.setOnlyAlertOnce (true)
			.setSmallIcon(Description.SmallIconResourceID)
			.addAction(Description.CancelIconResourceID, Description.CancelText, CancelIntent)
			.setNotificationSilent()
			.build();

		return notification;
	}

	//Gets the Notification Manager through the appropriate method based on build version
	public NotificationManager GetNotificationManager(@NonNull Context context)
	{
		if (Build.VERSION.SDK_INT >= Build.VERSION_CODES.M)
		{
			return context.getSystemService(NotificationManager.class);
		}
		else
		{
			return (NotificationManager)context.getSystemService(NOTIFICATION_SERVICE);
		}
	}

	private void CreateNotificationChannel(Context context, NotificationManager notificationManager, DownloadNotificationDescription Description)
	{
		if (Build.VERSION.SDK_INT >= Build.VERSION_CODES.O)
		{
			if (notificationManager != null)
			{
				//Don't create if it already exists
				NotificationChannel Channel = notificationManager.getNotificationChannel(Description.NotificationChannelID);
				if (Channel == null)
				{
					String ChannelName = Description.NotificationChannelName;
					if (ChannelName.equals("ue-downloadworker-channel"))
					{
						// check for an override for default channel name
						ChannelName = com.epicgames.unreal.LocalNotificationReceiver.getLocalizedResource(context, "UEBackgroundChannel", ChannelName);
					}
					
					Channel = new NotificationChannel(Description.NotificationChannelID, ChannelName, Description.NotificationChannelImportance);
					notificationManager.createNotificationChannel(Channel);
				}
			}
		}
	}
	
	public boolean ShouldCleanupDownloadDescriptorJSONFile()
	{
		return (IsWorkEndTerminal());
	}

	private boolean ShouldShowPercentage(Context context)
	{
		if (ShowPercentage == ESelectState.Unset)
		{
			SharedPreferences preferences = context.getSharedPreferences("BackgroundPreferences", context.MODE_PRIVATE);
			boolean bShow = preferences.getBoolean("bShowPercentage", true);
			ShowPercentage = bShow ? ESelectState.Enable : ESelectState.Disable;
		}
		return ShowPercentage == ESelectState.Enable;
	}

	private void ResetCellularPreference()
	{
		// Edit this to include Allways allow and only allow this time
		Context context = getApplicationContext();
		SharedPreferences preferences = context.getSharedPreferences("CellularNetworkPreferences", context.MODE_PRIVATE);
		SharedPreferences.Editor editor = preferences.edit();
		editor.putInt("AllowCellular", 0);
		editor.commit();
	}
	
	@Override
	public void OnDownloadStart(@NonNull String DestinationLocation, @NonNull String DebugString, long StartTimeUTC)
	{
	}

	//
	// DownloadCompletionListener Implementation
	//
	@Override
	public void OnDownloadProgress(@NonNull String DestinationLocation, long TotalBytesWritten, long BytesWrittenSinceLastCall)
	{
		if (!bGameThreadIsActive)
		{
			// avoid spamming the log with endless linker errors
			return;
		}

		try
		{
			nativeAndroidBackgroundDownloadOnProgress(DestinationLocation, TotalBytesWritten, BytesWrittenSinceLastCall);
		}
		catch (UnsatisfiedLinkError e)
		{
		}
	}

	//
	// This has to be called regularly, so that the notification description updates when there's no internet, or when we're waiting for cellular approval!
	// Normally, this gets called through Tick()'s call to mFetchManager.RequestGroupProgressUpdate
	//
	@Override
	public void OnDownloadGroupProgress(int GroupID, int Progress, boolean Indeterminate)
	{
		//For now all downloads are in the same GroupID, but in the future we will want a notification for each group ID 
		//and to upgate them separately here.
		UpdateNotification(Progress, Indeterminate);
	}
	
	@Override
	public void OnDownloadComplete(@NonNull String DestinationLocation, EDownloadCompleteReason CompleteReason)
	{
		if (!bGameThreadIsActive)
		{
			// avoid spamming the log with endless linker errors
			return;
		}

		boolean bWasSuccess = (CompleteReason == EDownloadCompleteReason.Success);
		try
		{
			nativeAndroidBackgroundDownloadOnComplete(DestinationLocation, bWasSuccess);
		}
		catch (UnsatisfiedLinkError e)
		{
		}
	}

	@Override
	public void OnDownloadMetrics(@NonNull String DestinationLocation, long TotalBytesDownloaded, long DownloadDuration, long DownloadStartTimeUTC, long DownloadEndTimeUTC)
	{
		if (!bGameThreadIsActive)
		{
			// avoid spamming the log with endless linker errors
			return;
		}

		try
		{
			nativeAndroidBackgroundDownloadOnMetrics(DestinationLocation, TotalBytesDownloaded, DownloadDuration, DownloadStartTimeUTC, DownloadEndTimeUTC);
		}
		catch (UnsatisfiedLinkError e)
		{
		}
	}
	
	@Override
	public void OnAllDownloadsComplete(boolean bDidAllRequestsSucceed)
	{	
		//If UE code has already provided a resolution then we do not need to handle this OnAllDownloadsComplete notification as 
		//this UEDownloadWorker is already in the process of stopping work. Also if we have already been force stopped, don't send
		//this to avoid sending this queued reply after we have already stopped work (possible to queue this during the tick before our force stop)
		if (!bReceivedResult && !bForceStopped)
		{
			UpdateNotification(100, false);

			if (bDidAllRequestsSucceed)
			{
				Context context = getApplicationContext();
				SharedPreferences preferences = context.getSharedPreferences("UEDownloadWorker", Context.MODE_PRIVATE);
				SharedPreferences.Editor editor = preferences.edit();
				editor.putString("AllDownloadsCompletedTime", String.format(Locale.getDefault(), "%f", System.currentTimeMillis() / 1000.0f));
				editor.commit();
			}

			// avoid spamming the log with endless linker errors
			if (bGameThreadIsActive)
			{
				try
				{
					nativeAndroidBackgroundDownloadOnAllComplete(bDidAllRequestsSucceed);
				}
				catch (UnsatisfiedLinkError e)
				{
				}
			}
		
			//If UE code didn't provide a result for the work in the above callback(IE: Engine isn't running yet, we are completely in background, etc.) 
			//then we need to still flag this Worker as completed and shutdown now that our task is finished
			if (!bReceivedResult)
			{
				if (bDidAllRequestsSucceed)
				{
					//Resetting cellular preference for the next run, you may want to allow cellular for this run, but not the next one.
					//This should be reworked and hooked up to the game code.
					ResetCellularPreference();
					SetWorkResult_Success();
				}
				//by default if UE didn't give us a behavior, lets just retry the download through the worker if one of the downloads failed
				else
				{
					SetWorkResult_Retry();
				}
			}
		}
	}

	@Override
	public void OnDownloadEnqueued(@NonNull DownloadDescription DownloadDescription)
	{
		Log.verbose("Enqueue success:" + DownloadDescription.DestinationLocation);

		bHasEnqueueHappened = true;
	}
	
	//Want to call our DownloadWorker version of OnWorkerStart
	@Override
	public void CallNativeOnWorkerStart(String WorkID)
	{
		try
		{
			nativeAndroidBackgroundDownloadOnWorkerStart(WorkID);
		}
		catch (UnsatisfiedLinkError e)
		{
		}
	}

	@Override
	public void CallNativeOnWorkerStop(String WorkID)
	{
		try
		{
			nativeAndroidBackgroundDownloadOnWorkerStop(WorkID);
		}
		catch (UnsatisfiedLinkError e)
		{
		}
	}

	//
	// Functions called by our UE c++ code on this object
	//
	public void PauseRequest(String DestinationLocation)
	{
		Log.debug("@*@ PauseRequest: " + DestinationLocation);

		boolean ShouldHandleCellular = false;
		if (NotificationDescription != null)
		{
			ShouldHandleCellular = true; // NotificationDescription.ShouldHandleCellular;
		}
		if (ShouldHandleCellular)
		{
			NetworkConnectivityClient.NetworkTransportType networkType = NetworkChangedManager.getInstance().networkTransportTypeCheck();
			if (networkType == NetworkConnectivityClient.NetworkTransportType.CELLULAR && !checkCellularAllowed())
			{
				bWaitingForCellularApproval = true;
			}
		}
		mFetchManager.PauseDownload(DestinationLocation, true);
	}
	
	public void ResumeRequest(String DestinationLocation)
	{
		Log.debug("@*@ ResumeRequest: " + DestinationLocation);

		// When the C++ resumes the request all network issues should be solved
		bWaitingForCellularApproval = false;
		mFetchManager.PauseDownload(DestinationLocation, false);
	}
	
	public void CancelRequest(String DestinationLocation)
	{
		mFetchManager.CancelDownload(DestinationLocation);
	}

	// This is a way of figuring out if the worker was spawned when the app was killed or not
	public static void AndroidThunkJava_GameThreadIsActive() 
	{
		bGameThreadIsActive = true;
	}
	
	//Native functions used to bubble up progress to native UE code
	native void nativeAndroidBackgroundDownloadOnWorkerStart(String WorkID);
	native void nativeAndroidBackgroundDownloadOnWorkerStop(String WorkID);
	native void nativeAndroidBackgroundDownloadOnProgress(String TaskID, long BytesWrittenSinceLastCall, long TotalBytesWritten);
	native void nativeAndroidBackgroundDownloadOnComplete(String TaskID, boolean bWasSuccess);
	native void nativeAndroidBackgroundDownloadOnMetrics(String TaskID, long TotalBytesDownloaded, long DownloadDuration, long DownloadStartTimeUTC, long DownloadEndTimeUTC);
	native void nativeAndroidBackgroundDownloadOnAllComplete(boolean bDidAllRequestsSucceed);
	native void nativeAndroidBackgroundDownloadOnTick();
	
	private ConnectivityManager connectivityManager = null;
	SharedPreferences preferences = null;
	
	SharedPreferences preferencesBackground = null;
	private String CurrentTaskFilename = "";

	private boolean bWaitingForCellularApproval = false;
	private NetworkConnectivityClient.Listener NetworkListener = null;
	private boolean bLostNetwork = false;
	private boolean bAirplaneMode = false;
	private MeteredType meteredType = MeteredType.UNMETERED;
	private volatile boolean bForceStopped = false;
	private DownloadQueueDescription QueueDescription = null;
	private volatile boolean bHasEnqueueHappened = false;
	static volatile FetchManagerInterface mFetchManager;
	private PendingIntent CancelIntent = null;
	private Intent ApproveIntent = null;
	private DownloadNotificationDescription NotificationDescription = null;
	private static boolean bGameThreadIsActive = false;

	private enum ESelectState
	{
		Unset,
		Disable,
		Enable
	};
	private ESelectState ShowPercentage = ESelectState.Unset;
}