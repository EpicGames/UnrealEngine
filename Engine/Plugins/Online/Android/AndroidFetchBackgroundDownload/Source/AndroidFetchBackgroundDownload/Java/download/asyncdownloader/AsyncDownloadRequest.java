// Copyright Epic Games, Inc. All Rights Reserved.

package com.epicgames.unreal.download.asyncdownloader;

import androidx.annotation.Nullable;

import java.io.File;
import java.util.ArrayList;
import java.util.List;
import java.util.Map;
import java.util.Objects;

final public class AsyncDownloadRequest {
    public final String Id;								// internal unique Id used for tracking
    public final String RequestId;						// optional request Id used for deduplication
    public final String[] Urls;							// cycle through these on failure
    public final int MaxRetries;						// total retry attempts across all URLs
    public final File TargetFile;
    public final boolean bResumeIfPossible;				// use Range: bytes=<currentLen>- if file exists, enabled by default
    public final @Nullable Map<String, String> Headers; // optional request headers per attempt
	public final int GroupID;
	public long TotalSize;

    private AsyncDownloadRequest(Builder b) {
        this.Id = Objects.requireNonNull(b.Id, "id");
        this.RequestId = b.RequestId;
        this.Urls = b.Urls;
        if (this.Urls.length == 0) 
			throw new IllegalArgumentException("urls must not be empty");
        this.MaxRetries = Math.max(0, b.MaxRetries);
        this.TargetFile = Objects.requireNonNull(b.TargetFile, "targetFile");
        this.bResumeIfPossible = b.bResumeIfPossible;
        this.Headers = (b.Headers == null || b.Headers.isEmpty()) ? null : Map.copyOf(b.Headers);
		this.GroupID = b.GroupID;
		this.TotalSize = b.TotalSize;
    }

    public static class Builder {
		private String Id;
		private String RequestId;
		private String[] Urls = new String[0];
		private int MaxRetries = 3;
		private File TargetFile;
		private boolean bResumeIfPossible = true;
		private Map<String, String> Headers;
		private int GroupID = 0;
		private long TotalSize = -1L;

        public Builder SetId(String Id) { 
			this.Id = Id; 
			return this; 
		}
        public Builder SetRequestId(String RequestId) { 
			this.RequestId = RequestId; 
			return this; 
		}
        public Builder SetUrls(String[] Urls) { 
			this.Urls = Urls; 
			return this; 
		}

        public Builder SetMaxRetries(int MaxRetries) { 
			this.MaxRetries = MaxRetries; 
			return this; 
		}
        public Builder SetTargetFile(File TargetFile) { 
			this.TargetFile = TargetFile; 
			return this; 
		}
        public Builder SetResumeIfPossible(boolean bResume) { 
			this.bResumeIfPossible = bResume; 
			return this; 
		}
        public Builder SetHeaders(Map<String, String> Headers) { 
			this.Headers = Headers; 
			return this; 
		}
		public Builder SetGroupID(int GroupID) {
			this.GroupID = GroupID;
			return this;
		}
		public Builder SetTotalSize(long TotalSize) {
			this.TotalSize = TotalSize;
			return this;
		}
        public AsyncDownloadRequest Build() { 
			return new AsyncDownloadRequest(this); 
		}
    }
}
