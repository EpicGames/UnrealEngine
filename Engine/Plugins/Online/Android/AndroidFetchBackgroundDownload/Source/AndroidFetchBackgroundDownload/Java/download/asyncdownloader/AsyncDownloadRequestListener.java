// Copyright Epic Games, Inc. All Rights Reserved.

package com.epicgames.unreal.download.asyncdownloader;

import androidx.annotation.NonNull;

import java.io.File;
import java.net.URL;

public interface AsyncDownloadRequestListener {
	void OnStart(@NonNull String Id, @NonNull String DebugString, long StartTimeUTC);
	void OnProgress(@NonNull String Id, long BytesDownloadedSoFar);
	void OnSuccess(@NonNull String Id, @NonNull File File);
	void OnError(@NonNull String Id, @NonNull Throwable Error, URL LastTriedUrl);
	void OnPaused(@NonNull String Id);
}
