// Copyright Epic Games, Inc. All Rights Reserved.

package com.epicgames.unreal.download.asyncdownloader;

import androidx.annotation.NonNull;
import androidx.annotation.Nullable;

import okhttp3.*;
import okio.BufferedSource;

import java.io.File;
import java.io.FileOutputStream;
import java.io.IOException;
import java.net.URL;
import java.nio.channels.FileChannel;
import java.nio.file.Files;
import java.nio.file.StandardOpenOption;
import java.time.Duration;
import java.util.*;
import java.util.concurrent.*;
import java.util.concurrent.atomic.AtomicBoolean;

import com.epicgames.unreal.GameActivity;
import com.epicgames.unreal.Logger;

public class AsyncDownloader {

	// ---------------------------------------------------------------------------------------------------------------------
	// public interface
	// ---------------------------------------------------------------------------------------------------------------------	
	
	public AsyncDownloader(boolean AllowHTTP2, int MaxConcurrent) {
		//Log.debug("AsyncDownloader: AllowHTTP2=" + AllowHTTP2 + " | MaxConcurrent=" + MaxConcurrent);
		this.Dispatcher = new Dispatcher();
		this.Dispatcher.setMaxRequests(Math.max(1, MaxConcurrent));
		this.Dispatcher.setMaxRequestsPerHost(Math.max(1, MaxConcurrent));
		this.Client = new OkHttpClient.Builder()
				.dispatcher(Dispatcher)
				.callTimeout(Duration.ofMinutes(30))
				.connectTimeout(Duration.ofSeconds(20))
				.readTimeout(Duration.ofMinutes(5))
				.protocols(AllowHTTP2 ? List.of(Protocol.HTTP_2, Protocol.HTTP_1_1) : List.of(Protocol.HTTP_1_1))
				.build();
	}

	public synchronized void SetMaxConcurrent(int MaxConcurrent) {
		//Log.debug("SetMaxConcurrent: MaxConcurrent=" + MaxConcurrent);
		Dispatcher.setMaxRequests(Math.max(1, MaxConcurrent));
		Dispatcher.setMaxRequestsPerHost(Math.max(1, MaxConcurrent));
	}

	public synchronized void SetAsyncDownloaderListener(AsyncDownloaderListener Listner) { 
		this.DownloaderListener = Listner;
	}

	public synchronized boolean Submit(AsyncDownloadRequest Request, AsyncDownloadRequestListener Listener) {
		//Log.debug("Submit: Id=" + Request.Id);
		DownloadEntry Existing = Entries.get(Request.Id);
		if (Existing == null) {
			DownloadEntry Entry = new DownloadEntry(Request, Listener);
			Entries.put(Request.Id, Entry);
			StartOrQueue(Entry);
			return true;
		}
		Existing.Listener = Listener;
		//Log.debug("Submit: pre-Existing, updated Listener");
		return false;
	}

	public synchronized void Pause(String Id, boolean userPaused) {
		//Log.debug("Pause: Id=" + Id);
		DownloadEntry Entry = Entries.get(Id);
		if (Entry == null) {
			return;
		}
		if (Entry.State == EState.PAUSED) {
			//Log.debug("Pause: Id=" + Id + " is already paused, ignoring call");
			return;
		}
		if (userPaused)
		{
			Entry.UserPaused = true;
		}
		Call c = Entry.currentCall;
		if (c != null) {
			c.cancel();
		}
		Entry.State = EState.PAUSED;
		Entry.Listener.OnPaused(Id);
		Entry.Attempts = 0;
	}

	public synchronized void PauseAll() {
		if (PausedAll.get()) {
			return;
		}
		//Log.debug("PauseAll");
		PausedAll.set(true);
		for (DownloadEntry Entry : Entries.values()) {
			Pause(Entry.Request.Id, false);
		}
	}

	public synchronized void Resume(String Id, boolean userResumed) {
		//Log.debug("Resume: Id=" + Id);
		DownloadEntry Entry = Entries.get(Id);
		if (Entry == null) {
			return;
		}
		if (Entry.State != EState.PAUSED) {
			//Log.debug("Resume: Id=" + Id + " not paused, ignoring call");
			return;
		}
		if (userResumed)
		{
			Entry.UserPaused = false;
		}
		StartOrQueue(Entry);
	}

	public synchronized void ResumeAll() {
		if (!PausedAll.get()) {
			return;
		}
		//Log.debug("ResumeAll");
		PausedAll.set(false);
		for (DownloadEntry Entry : Entries.values()) {
			Resume(Entry.Request.Id, false);
		}
	}

	public synchronized void Cancel(String Id) {
		//Log.debug("Cancel: Id=" + Id);
		DownloadEntry Entry = Entries.remove(Id);
		if (Entry == null) {
			return;
		}
		Call c = Entry.currentCall;
		if (c != null) {
			c.cancel();
		}
		Entry.State = EState.CANCELLED;
		CheckIfAllDownloadsCompleted();
	}

	public synchronized void CancelAll() {
		//Log.debug("CancelAll");

		// do not fire the AllDownloadsComplete if we're cancelling them all
		bAllowCheckIfAllDownloadsCompleted = false;
		try {
			for (String Id : new ArrayList<>(Entries.keySet())) {
				Cancel(Id);
			}
		} finally {
			bAllowCheckIfAllDownloadsCompleted = true;
		}
	}

	public synchronized void Retry(String Id) {
		//Log.debug("Retry: Id=" + Id);

		DownloadEntry OldEntry = Entries.remove(Id);
		if (OldEntry == null) {
			return;
		}

		AsyncDownloadRequest Req = OldEntry.Request;
		AsyncDownloadRequestListener Listener = OldEntry.Listener;

		OldEntry.Cancelled = true;
		Call c = OldEntry.currentCall;
		if (c != null) {
			c.cancel();	
		}
		OldEntry.State = EState.CANCELLED;

		DownloadEntry NewEntry = new DownloadEntry(Req, Listener);
		NewEntry.UserPaused = false;
		Entries.put(Req.Id, NewEntry);

		// try it again :)
		StartOrQueue(NewEntry);
	}

	public synchronized float GetGroupProgress(int GroupID) {
		long TotalBytes = 0;
		long DownloadedBytes = 0;

		for (DownloadEntry e : Entries.values()) {
			if (e.Request.GroupID != GroupID) {
				continue;
			}
			if (e.Request.TotalSize <= 0) {
				return -1f;
			}

			if (e.State == EState.COMPLETED) {
				DownloadedBytes += e.Request.TotalSize;
			} else {
				DownloadedBytes += e.DownloadedSoFar;
			}
			TotalBytes += e.Request.TotalSize;
		}

		if (TotalBytes == 0L) {
			return -1f;
		}
		return Math.min(1f, (float)DownloadedBytes / (float)TotalBytes);
	}

	public synchronized String[] GetRunningDestinationLocations() {
		ArrayList<String> result = new ArrayList<>();
		for (Map.Entry<String, DownloadEntry> entry : Entries.entrySet()) {
			DownloadEntry e = entry.getValue();
			// EState.RUNNING is set at enqueue time, before OkHttp's Dispatcher assigns a connection.
			// With MaxActiveDownloads, entries beyond the limit are queued internally by OkHttp but still
			// marked RUNNING. Use two signals to confirm a real connection:
			//   - bResponseReceived: HTTP response headers received (connection established)
			//   - DownloadedSoFar > 0: body data is flowing
			if (e.State == EState.RUNNING && (e.bResponseReceived || e.DownloadedSoFar > 0)) {
				result.add(entry.getKey());
			}
		}
		return result.toArray(new String[0]);
	}

	// ---------------------------------------------------------------------------------------------------------------------
	// private data + methods
	// ---------------------------------------------------------------------------------------------------------------------

	private void DeleteFileIfExists(File file) {
		try {
			Files.deleteIfExists(file.toPath());
		} catch (IOException ignored) {
		}
	}

	private void StartOrQueue(DownloadEntry Entry) {
		//Log.debug("StartOrQueue: Id=" + Entry.Request.Id + " | TotalSize=" + Entry.Request.TotalSize);
		if (Entry.State == EState.COMPLETED) {
			//Log.debug("StartOrQueue: State set to completed for id " + Entry.Request.Id);
			return;
		}
		if (Entry.State == EState.CANCELLED) {
			//Log.debug("StartOrQueue: State set to cancelled for id " + Entry.Request.Id);
			return;
		}
		if (Entry.UserPaused) {
			//Log.debug("StartOrQueue: State set to user-paused for id " + Entry.Request.Id);
			return;
		}
		if (PausedAll.get()) {
			//Log.debug("StartOrQueue: PausedAll set to true, not starting on Id " + Entry.Request.Id);
			return;
		}
		//Log.debug("StartOrQueue: started work on Id " + Entry.Request.Id);

		try {
			final File file = Entry.Request.TargetFile;
			Files.createDirectories(file.getParentFile().toPath());
			long CurrentLen = (file.exists() && Entry.Request.bResumeIfPossible) ? file.length() : 0L;

			URL NextUrl = new URL(Entry.Request.Urls[Entry.UrlIndex % Entry.Request.Urls.length]);
			Request.Builder rb = new Request.Builder().url(NextUrl);
			if (Entry.Request.Headers != null) {
				for (Map.Entry<String, String> h : Entry.Request.Headers.entrySet()) {
					rb.addHeader(h.getKey(), h.getValue());
				}
			}

			// Ask Akamai edges to surface cache state in response headers (debug builds only).
			if (!GameActivity.IS_SHIPPING_CONFIG && NextUrl.getHost() != null && NextUrl.getHost().endsWith(".akamaized.net")) {
				rb.header("Pragma", "akamai-x-cache-on, akamai-x-cache-remote-on, akamai-x-check-cacheable");
			}

			if (CurrentLen > 0) {
				if (CurrentLen < Entry.Request.TotalSize)
				{
					rb.header("Range", "bytes=" + CurrentLen + "-");
					//Log.debug("StartOrQueue: requesting partial content for id " + Entry.Request.Id + " | Offset " + CurrentLen);
				} else if (CurrentLen == Entry.Request.TotalSize) {
					synchronized(this) {
						if (Entries.remove(Entry.Request.Id) != null)
						{
							Entry.State = EState.COMPLETED;
							Entry.DownloadedSoFar = CurrentLen;
							Entry.Listener.OnStart(Entry.Request.Id, "Completed", System.currentTimeMillis());
							Entry.Listener.OnProgress(Entry.Request.Id, CurrentLen);
							Entry.Listener.OnSuccess(Entry.Request.Id, file);
						}
					}
					CheckIfAllDownloadsCompleted();
					return;
				} else if (Entry.Request.TotalSize > 0) {
					// be on the safe side: delete the file completely if it is bigger than we expected
					Log.error("StartOrQueue: requesting full file because current file size is too big (" + CurrentLen + " > " + Entry.Request.TotalSize + ")");
					CurrentLen = 0;
					DeleteFileIfExists(file);
				}
			}

			Request Request = rb.build();
			Entry.State = EState.RUNNING;
			Entry.bResponseReceived = false;
			Entry.DownloadedSoFar = CurrentLen;

			Call Call = Client.newCall(Request);
			Entry.currentCall = Call;
			final Object syncRoot = this;
			Call.enqueue(new Callback() {
				@Override public void onFailure(@NonNull Call Call, @NonNull IOException Except) {
					Log.error("onFailure: error: failed to begin downloading | Id=" + Entry.Request.Id + " | Exception=" + Except);
					HandleAttemptFailure(Entry, Except, NextUrl);
				}

				@Override public void onResponse(@NonNull Call Call, @NonNull Response Response) {
					Log.debug("onResponse: begin downloading | Id=" + Entry.Request.Id);
					Entry.bResponseReceived = true;

					String DebugString = "";
					if (!GameActivity.IS_SHIPPING_CONFIG)
					{
						// Keep in sync with Engine/Source/Runtime/Apple/BackgroundHttpIOS/Private/BackgroundURLSessionHandler.cpp
						String Cache = Response.header("X-Epic-Cache");
						if (Cache == null)
						{
							Cache = Response.header("X-Cache");
						}
						if (Cache == null)
						{
							Cache = Response.header("CF-Cache-Status");
						}

						String Age = Response.header("Age");
						String Via = Response.header("Via");
						String Server = Response.header("Server");
						StringBuilder Builder = new StringBuilder();

						if (Cache != null)
						{
							Builder.append("Cache:").append(Cache);
						}
						else if (Age != null)
						{
							Builder.append("Age:").append(Age);
						}
						else if (Via != null)
						{
							Builder.append("Via:").append(Via);
						}
						else if (Server != null)
						{
							Builder.append("Srv:").append(Server);
						}
						else
						{
							Builder.append("NoCacheHdr");
						}
						DebugString = Builder.toString();
					}
					Entry.Listener.OnStart(Entry.Request.Id, DebugString, System.currentTimeMillis());

					try (Response response = Response) {
						Log.debug("onResponse: protocol=" + response.protocol() + " | Id=" + Entry.Request.Id);
						int Code = response.code();
						if (!response.isSuccessful()) {
							Log.error("onResponse: error: not successful | Id=" + Entry.Request.Id + " | Code=" + Code);
							HandleAttemptFailure(Entry,
									new IOException("HTTP " + Code + " " + response.message()),
									NextUrl);
							return;
						}

						long ContentLen = response.body() != null 
							? response.body().contentLength() 
							: Entry.Request.TotalSize;
						boolean bIsPartial = Code == 206; // Partial Content means server honored Range
						boolean bShouldAppend = bIsPartial && Entry.Request.bResumeIfPossible && file.exists();
						if (!bShouldAppend) {
							// always remove the file if we're not appending
							DeleteFileIfExists(file);

							// but if this is partial content, fail the request, otherwise we'd be putting a half-finished file on disk
							if (bIsPartial) {
								Log.error("onResponse: error: received partial content, but not able to append, cannot put half file on disk");
								HandleAttemptFailure(Entry,
										new IOException("HTTP " + Code + " " + response.message()),
										NextUrl);
								return;
							}
						}

						// check if the total matches what we expect
						long FileOnDiskLen;
						long TotalContentLen;
						if (bShouldAppend) {
							FileOnDiskLen = file.length();
							TotalContentLen = ContentLen >= 0 ? FileOnDiskLen + ContentLen : ContentLen;
						} else {
							FileOnDiskLen = 0;
							TotalContentLen = ContentLen;
						}
						if (TotalContentLen != Entry.Request.TotalSize && Entry.Request.TotalSize > 0) {
							Log.error("onResponse: error: content length mismatch: TotalContentLen="+TotalContentLen+" != TotalExpectedSize="+Entry.Request.TotalSize+" | ContentLen="+ContentLen+" | FileOnDiskLen=" + FileOnDiskLen);
							HandleAttemptFailure(Entry,
									new IOException("HTTP " + Code + " " + response.message()),
									NextUrl);
							return;
						}
						if (TotalContentLen > 0) {
							Entry.Request.TotalSize = TotalContentLen;
						}

						// all is good, start downloading
						try (BufferedSource Source = Objects.requireNonNull(response.body()).source();
							 FileChannel Channel = FileChannel.open(file.toPath(), StandardOpenOption.CREATE, StandardOpenOption.WRITE)) {

							long WritePos = FileOnDiskLen;
							if (!bShouldAppend) {
								Channel.truncate(0L);
							}

							while (!Source.exhausted()) {
								long NumBytesRead = Channel.transferFrom(Source, WritePos, 64 * 1024);

								WritePos += NumBytesRead;
								Entry.DownloadedSoFar = WritePos;
								Entry.Listener.OnProgress(Entry.Request.Id, WritePos);
							}
						}
					} catch (Throwable Error) {
						HandleAttemptFailure(Entry, Error, NextUrl);
						return;
					}

					Log.debug("onResponse: download completed | Id=" + Entry.Request.Id);
					synchronized(syncRoot) {
						if (Entries.remove(Entry.Request.Id) != null)
						{
							Entry.State = EState.COMPLETED;
							Entry.currentCall = null;
							Entry.Listener.OnSuccess(Entry.Request.Id, file);
						}
					}
					CheckIfAllDownloadsCompleted();
				}
			});

		} catch (Throwable t) {
			HandleAttemptFailure(Entry, t, null);
		}
	}

	private synchronized void HandleAttemptFailure(@NonNull DownloadEntry Entry, @NonNull Throwable Error, URL LastUrl) {
		//Log.debug("HandleAttemptFailure: Id=" + Entry.Request.Id);
		Entry.currentCall = null;
		if (Entry.UserPaused || PausedAll.get()) {
			//Log.debug("HandleAttemptFailure: Id=" + Entry.Request.Id + " | User parked or PausedAll true, parking");
			Entry.State = EState.PAUSED;
			Entry.Listener.OnPaused(Entry.Request.Id);
			CheckIfAllDownloadsCompleted();
			return;
		}

		int Attempt = ++Entry.Attempts;
		Entry.UrlIndex = (Entry.UrlIndex + 1) % Entry.Request.Urls.length;

		final boolean bHaveRetriesRemaining = Attempt <= Entry.Request.MaxRetries;
		if (bHaveRetriesRemaining) {
			//Log.debug("HandleAttemptFailure: Id=" + Entry.Request.Id + " | trying next URL");
			schedule(() -> { synchronized(this) { StartOrQueue(Entry); } }, 1000L);
		} else {
			//Log.debug("HandleAttemptFailure: Id=" + Entry.Request.Id + " | out of retries: failing download");

			if (Entries.remove(Entry.Request.Id) != null)
			{
				Entry.State = EState.FAILED;
				Entry.Listener.OnError(Entry.Request.Id, Error, LastUrl);
			}
			CheckIfAllDownloadsCompleted();
		}
	}

	private synchronized void CheckIfAllDownloadsCompleted() {
		Log.debug("CheckIfAllDownloadsCompleted: Entries.size()=" + Entries.size());
		if (!bAllowCheckIfAllDownloadsCompleted) {
			return;
		}
		if (!Entries.isEmpty()) {
			return;
		}
		//Log.debug("CheckIfAllDownloadsCompleted: done, firing onAllCompleted");
		AsyncDownloaderListener Listener = DownloaderListener;
		if (Listener != null) {
			Log.debug("CheckIfAllDownloadsCompleted: firing listener");
			Listener.OnAllCompleted();
		} else {
			//Log.debug("CheckIfAllDownloadsCompleted: no listener set!");
		}
	}

	private static final ScheduledExecutorService Scheduler = Executors.newSingleThreadScheduledExecutor(r -> {
		Thread thread = new Thread(r, "AsyncDownloaderRetryThread");
		thread.setDaemon(true);
		return thread;
	});

	private static void schedule(Runnable RunMe, long DelayMs) {
		Scheduler.schedule(RunMe, DelayMs, TimeUnit.MILLISECONDS);
	}

	private enum EState { PENDING, RUNNING, PAUSED, COMPLETED, FAILED, CANCELLED }

	private final OkHttpClient Client;
	private final Dispatcher Dispatcher;
	private final ConcurrentHashMap<String, DownloadEntry> Entries = new ConcurrentHashMap<>();
	private final AtomicBoolean PausedAll = new AtomicBoolean(false);								// if this is set, all new downloads will ALSO be paused!
	private volatile @Nullable AsyncDownloaderListener DownloaderListener;
	private Logger Log = new Logger("UE", "AsyncDownloader");
	private boolean bAllowCheckIfAllDownloadsCompleted = true;

	private static final class DownloadEntry {
		final AsyncDownloadRequest Request;
		volatile AsyncDownloadRequestListener Listener;
		volatile EState State = EState.PENDING;
		volatile @Nullable Call currentCall;
		volatile int Attempts = 0;				// total attempts across all URLs
		volatile int UrlIndex = 0;				// which URL we'll try next
		volatile long DownloadedSoFar = 0L;		// tracked for progress (file length + session)
		volatile boolean bResponseReceived = false; // true once HTTP response headers are received (real connection)
		volatile boolean UserPaused = false;	// true if pause(Id) was requested
		volatile boolean Cancelled = false;

		DownloadEntry(AsyncDownloadRequest Request, AsyncDownloadRequestListener Listener) {
			this.Request = Request;
			this.Listener = Listener;
		}
	}

}
