// Copyright Epic Games, Inc. All Rights Reserved.

package com.epicgames.unreal.download.asyncdownloader;

public interface AsyncDownloaderListener {
    public void OnAllCompleted();
}
