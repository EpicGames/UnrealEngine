// Copyright Epic Games, Inc. All Rights Reserved.
package com.epicgames.unreal.download.datastructs;

//WARNING: We don't import org.json.JSONArray OR org.json.simple.JSONArray as we use both in this file
import org.json.JSONException;
import org.json.JSONObject;

import org.json.simple.parser.JSONParser;

import java.io.File;
import java.io.IOException;
import java.io.FileWriter;
import java.io.FileReader;

import java.util.ArrayList;
import java.util.List;
import java.util.stream.IntStream;

import com.epicgames.unreal.download.DownloadProgressListener;

//Helper class that holds the necessary data to describe a single URL request and loading/saving that request to JSON strings.
public class DownloadDescription
{
	//
	//Serialized Members -- Pulled from WorkerParameters
	//
	public String[] URLs;
	public String RequestID = null;
	public String DestinationLocation = null;
	public int RequestPriority = 0;
	public int GroupID = 0;
	//How many retries we get across all URLs
	public int MaxRetryCount = 0;
	//How many times we retry a single URL before moving on to the next one
	public int IndividualURLRetryCount = 0;
	//If our download has been completed and thus shouldn't be redownloaded
	public boolean bHasCompleted = false;

	//
	//Serliazed AND state Members -- Initial state is pulled from WorkerParameters but further
	//changes are made to track download information
	//

	//If our download is currently paused.
	public boolean bIsPaused = false;

	//Non-serialized members -- Used to track download information
	//
	public int CurrentRetryCount = 0;
	public int CachedFetchID = 0;
	
	public boolean bIsCancelled = false;

	public int PreviousDownloadPercent = 0;
	public long PreviousDownloadedBytes = 0;
	public long TotalBytesNeeded = 0;
	public long TotalDownloadedBytes = 0;
	public DownloadProgressListener ProgressListener = null;

	// Fields used to calculate the total download time
	public long DownloadStartTime = 0;
	public long DownloadPauseTime = 0;
	public long TotalPausedTime = 0;

	// Fields used for final metrics
	public long DownloadDuration = 0;
	public long DownloadStartTimeUTC = 0;
	public long DownloadEndTimeUTC = 0;
	
	public DownloadDescription()
	{
	}

	public DownloadDescription(String RequestID, String DestinationLocation, int MaxRetryCount, int IndividualURLRetryCount, int RequestPriority, int GroupID, String[] URLs, boolean bIsPaused, long TotalBytesNeeded)
	{
		this.RequestID = RequestID;
		this.DestinationLocation = DestinationLocation;
		this.MaxRetryCount = MaxRetryCount;
		this.IndividualURLRetryCount = IndividualURLRetryCount;
		this.RequestPriority = RequestPriority;
		this.GroupID = GroupID;
		this.URLs = URLs;
		this.bIsPaused = bIsPaused;
		this.TotalBytesNeeded = TotalBytesNeeded;
	}
	
	public static ArrayList<DownloadDescription> BuildDescriptionArray()
	{
		return new ArrayList<DownloadDescription>();
	}
	
	//Helper function that creates a JSON representation of the supplied ArrayList<DownloadDescription>
	//True / false returned based on if write succeeds
	public static boolean WriteDownloadDescriptionListToFile(String FileName, ArrayList<DownloadDescription> DownloadDescriptions)
	{
		boolean bSucceeded = true;
		
		//Attempt to write out JSONArray string to file
		FileWriter JsonFile = null;
		{
			try
			{
				File NewFile = new File (FileName);
				
				//Need to create the directory / files
				if (!NewFile.exists()) 
				{
					//create parent directories if needed
					if (!NewFile.getParentFile().exists())
					{
						NewFile.getParentFile().mkdirs();
					}
					
					//creates actual file
					NewFile.createNewFile();
				}
				//Don't append because we want to stomp the file
				JsonFile = new FileWriter(NewFile, false);
				
				//Can actually write the json array out to the file now
				JsonFile.write("[");
				for (int DescriptionIndex = 0; DescriptionIndex < DownloadDescriptions.size(); ++DescriptionIndex)
				{
					if (DescriptionIndex > 0) {
						JsonFile.write(",\n");
					}
					DownloadDescription Description = DownloadDescriptions.get(DescriptionIndex);
					JsonFile.write(Description.ToJSON().toString());
				}
				JsonFile.write("]\n");
			}
			catch (IOException e)
			{
				bSucceeded = false;
				e.printStackTrace();
			}
			catch (Exception e)
			{
				bSucceeded = false;
				e.printStackTrace();
			}
			finally
			{
				//even if our write fails, we still want to try and flush and close the file handle
				if (null != JsonFile) 
				{
					try
					{
						JsonFile.flush();
						JsonFile.close();		
					}
					catch (IOException e)
					{
						bSucceeded = false;
						e.printStackTrace();
					}
				}
			}
		}
		
		return bSucceeded;
	}

	//Parses the supplied filename to try and get a JSON String output of the contents of the file.
	public static String GetDownloadDescriptionJSONArrayFromFile(String FileName)
	{
		String JSONOutputString = null;
		
		JSONParser Parser = new JSONParser();
		FileReader JsonFile = null;
		try
		{
			JsonFile = new FileReader(FileName);
			Object Obj = Parser.parse(JsonFile);
			org.json.simple.JSONArray JsonArray = (org.json.simple.JSONArray)Obj;
			
			JSONOutputString = JsonArray.toString();
		}
		catch (Exception e)
		{
			e.printStackTrace();
		}
		finally
		{
			if (null != JsonFile)
			{
				try 
				{
					JsonFile.close();
				}
				catch(Exception e)
				{
					e.printStackTrace();
				}
			}
		}
		
		return JSONOutputString;
	}

	//Returns a new IndividualDownloadDescroption based on the JSONString. Returns null if error
	public static DownloadDescription FromJSON(String JSONString)
	{
		DownloadDescription ReturnedDescription = new DownloadDescription();
		try
		{
			JSONObject jObject = new JSONObject(JSONString);
			
			if (false == jObject.isNull(RequestIDKey))
			{
				ReturnedDescription.RequestID = jObject.optString(RequestIDKey);
			}
			
			if (false == jObject.isNull(DestinationLocationKey))
			{
				ReturnedDescription.DestinationLocation = jObject.optString(DestinationLocationKey);
			}
			
			if (false == jObject.isNull(MaxRetryCountKey))
			{
				ReturnedDescription.MaxRetryCount = jObject.optInt(MaxRetryCountKey);
			}
			
			if (false == jObject.isNull(IndividualURLRetryCountKey))
			{
				ReturnedDescription.IndividualURLRetryCount = jObject.optInt(IndividualURLRetryCountKey);
			}
			
			if (false == jObject.isNull(RequestPriorityKey))
			{
				ReturnedDescription.RequestPriority = jObject.optInt(RequestPriorityKey);
			}
			
			if (false == jObject.isNull(GroupIDKey))
			{
				ReturnedDescription.GroupID = jObject.optInt(GroupIDKey);
			}
			
			if (false == jObject.isNull(URLKey))
			{
				org.json.JSONArray JSONURLs = jObject.getJSONArray(URLKey);
				ReturnedDescription.URLs = IntStream.range(0, JSONURLs.length()).mapToObj(JSONURLs::optString).toArray(String[]::new);
			}

			if (false == jObject.isNull(bHasCompletedKey))
			{
				ReturnedDescription.bHasCompleted = jObject.optBoolean(bHasCompletedKey);
			}

			if (false == jObject.isNull(bIsPausedKey))
			{
				ReturnedDescription.bIsPaused = jObject.optBoolean(bIsPausedKey);
			}

			if (false == jObject.isNull(TotalBytesNeededKey))
			{
				ReturnedDescription.TotalBytesNeeded = jObject.optLong(TotalBytesNeededKey);
			}
		}
		catch( JSONException e )
		{
			e.printStackTrace();
			ReturnedDescription = null;
		}
		finally
		{
			return ReturnedDescription;
		}
	}
	
	//Returns a String descibing this DownloadDescription in JSON.  Returns null if error
	public String ToJSON()
	{
		JSONObject JSONObj = new JSONObject();
		try 
		{
			org.json.JSONArray URLArray = new org.json.JSONArray();
			for (String URL : URLs) {
				URLArray.put(URL);
			}

			JSONObj.put(URLKey, URLArray);
			JSONObj.put(RequestIDKey, RequestID);
			JSONObj.put(DestinationLocationKey, DestinationLocation);
			JSONObj.put(MaxRetryCountKey, MaxRetryCount);
			JSONObj.put(IndividualURLRetryCountKey, IndividualURLRetryCount);
			JSONObj.put(RequestPriorityKey, RequestPriority);
			JSONObj.put(GroupIDKey, GroupID);
			JSONObj.put(bHasCompletedKey, bHasCompleted);
			JSONObj.put(TotalBytesNeededKey, TotalBytesNeeded);
		}
		catch( JSONException e )
		{
			e.printStackTrace();
			JSONObj = null;
		}
		finally 
		{
			return JSONObj.toString();
		}
	}
	
	//This equals function is only really used to determine if we need to replace an existing DownloadDescription with a new one
	//that comes in through new WorkerParameters. As such not all values are compared (non-serialized ones are definitely skipped as well as some others)
	//TLDR: If we wouldn't want to delete the download and recreate it when a value changes, it isn't compared here.
	@Override
	public boolean equals(Object Other)
	{
		//comparing 2 of the same objects in memory so must be equal
		if (Other == this)
		{
			return true;
		}

		if (false == (Other instanceof DownloadDescription))
		{
			return false;
		}

		DownloadDescription CastOther = (DownloadDescription) Other;
		
		return ((CastOther.RequestID == this.RequestID) &&
				(this.MaxRetryCount == CastOther.MaxRetryCount) &&
				(this.bHasCompleted == CastOther.bHasCompleted) &&
				(this.IndividualURLRetryCount == CastOther.IndividualURLRetryCount) &&
				(this.RequestPriority == CastOther.RequestPriority) &&
				(this.HasNewURLs(CastOther)) &&
				(this.HasNewDestinationLocation(CastOther)));
	}
	
	//Requires the URLs lists to match exactly
	public boolean HasExactSameURLs(DownloadDescription Other)
	{
		return ((this.URLs.length == Other.URLs.length) && (!this.HasNewURLs(Other)));
	}
	
	//This only requires that our DestinationLocation has all the same URLs as the Other, but it can have more.
	public boolean HasNewURLs(DownloadDescription Other)
	{
		return !List.of(this.URLs).containsAll(List.of(Other.URLs));
	}
	
	public boolean HasNewDestinationLocation(DownloadDescription Other)
	{
		return (!this.DestinationLocation.contentEquals(Other.DestinationLocation));
	}
	
	//Our keys used for serializing / deserializing our members to/from JSON
	//Note: private and not in our DownloadWorkerParameterKeys as they aren't directly in our WorkerParameters and only have to be known internally
	private static final String URLKey = "URLs";
	private static final String RequestIDKey = "RequestID";
	private static final String DestinationLocationKey = "DestLocation";
	private static final String MaxRetryCountKey = "MaxRetryCount";
	private static final String IndividualURLRetryCountKey = "IndividualURLRetryCount";
	private static final String RequestPriorityKey = "RequestPriority";
	private static final String GroupIDKey = "GroupId";
	private static final String bHasCompletedKey = "bHasCompleted";
	private static final String bIsPausedKey = "bIsPaused";
	private static final String TotalBytesNeededKey = "TotalBytesNeeded";
}