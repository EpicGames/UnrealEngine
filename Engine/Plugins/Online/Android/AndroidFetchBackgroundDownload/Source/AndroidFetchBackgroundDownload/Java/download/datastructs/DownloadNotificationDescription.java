// Copyright Epic Games, Inc. All Rights Reserved.
package com.epicgames.unreal.download.datastructs;

import android.app.NotificationManager;
import android.content.Context;
import android.content.SharedPreferences;

import androidx.annotation.NonNull;
import androidx.annotation.Nullable;
import androidx.work.Data;

import com.epicgames.unreal.LocalNotificationReceiver;
import com.epicgames.unreal.Logger;

import java.util.Map;

//Helper class that stores all the needed information for a Notification in one object and handles parsing and caching defaults stored in the WorkerParameters
public class DownloadNotificationDescription
{
	//Constructor that parses good defaults from our Worker InputData
	public DownloadNotificationDescription(@NonNull Data data, @NonNull Context context, @Nullable Logger Log)
	{
		this(data.getKeyValueMap(), context, Log);
	}

	public DownloadNotificationDescription(@NonNull SharedPreferences preferences, @NonNull Context context, @Nullable Logger Log)
	{
		this(preferences.getAll(), context, Log);
	}

	public DownloadNotificationDescription(@NonNull Map<String, ?> data, @NonNull Context context, @Nullable Logger Log)
	{
		//Set values that are just raw defaults that we expect to be manually overriden
		{
			CurrentProgress = 0;
			Indeterminate = true;
		}
					
		//Load notification channel information
		{
			NotificationChannelID = data.get(DownloadWorkerParameterKeys.NOTIFICATION_CHANNEL_ID_KEY) instanceof String value ? value : null;
			if (null == NotificationChannelID) 
			{
				NotificationChannelID = "ue-downloadworker-channel-id";
			}

			NotificationChannelName = data.get(DownloadWorkerParameterKeys.NOTIFICATION_CHANNEL_NAME_KEY) instanceof String value ? value : null;
			if (null == NotificationChannelName) 
			{
				NotificationChannelName = "ue-downloadworker-channel";
			}

			NotificationChannelImportance = data.get(DownloadWorkerParameterKeys.NOTIFICATION_CHANNEL_IMPORTANCE_KEY) instanceof Integer value ? value : NotificationManager.IMPORTANCE_DEFAULT;
		}
		
		//Load notification base information
		{
			//Loads from data or defaults to a random number (that is hopefully unique) as this can NOT be set to 0 for SetForeground notifications!

			NotificationID = data.get(DownloadWorkerParameterKeys.NOTIFICATION_ID_KEY) instanceof Integer value ? value : DownloadWorkerParameterKeys.NOTIFICATION_DEFAULT_ID_KEY;

			if (NotificationID == 0)
			{
				if (null != Log) 
				{
					Log.error("Invalid NotificationID for notification! Will not be able to activate as a foreground service correctly!");
				}
			}
		}

		//Configuration 
		{
			ShouldHandleCellular = data.get(DownloadWorkerParameterKeys.CELLULAR_HANDLE_KEY) instanceof Boolean value ? value : false;

			ShouldShowPercentage = data.get(DownloadWorkerParameterKeys.SHOW_PERCENTAGE_KEY) instanceof Boolean value ? value : false;
		}

		//Load notification content information
		{
			TitleText = data.get(DownloadWorkerParameterKeys.NOTIFICATION_CONTENT_TITLE_KEY) instanceof String value ? value : null;
			if (null == TitleText) 
			{
				TitleText = "Downloading";
			}

			ContentText = data.get(DownloadWorkerParameterKeys.NOTIFICATION_CONTENT_TEXT_KEY) instanceof String value ? value : null;
			if (null == ContentText) 
			{
				ContentText = "Download in Progress";
			}

			ContentCompleteText = data.get(DownloadWorkerParameterKeys.NOTIFICATION_CONTENT_COMPLETE_TEXT_KEY) instanceof String value ? value : null;
			if (null == ContentCompleteText) 
			{
				ContentCompleteText = "Complete";
			}
			
			CancelText = data.get(DownloadWorkerParameterKeys.NOTIFICATION_CONTENT_CANCEL_DOWNLOAD_TEXT_KEY) instanceof String value ? value : null;
			if (null == CancelText) 
			{
				CancelText = "Cancel";
			}

			NoInternetAvailable = data.get(DownloadWorkerParameterKeys.NOTIFICATION_CONTENT_NO_INTERNET_TEXT_KEY) instanceof String value ? value : null;
			if (null == NoInternetAvailable)
			{
				NoInternetAvailable = "No Internet Available";
			}

			AirplaneModeText = data.get(DownloadWorkerParameterKeys.NOTIFICATION_CONTENT_AIRPLANE_MODE_TEXT_KEY) instanceof String value ? value : null;
			if (null == AirplaneModeText)
			{
				AirplaneModeText = "Airplane Mode Enabled";
			}
			
			DataSaverEnabledText = data.get(DownloadWorkerParameterKeys.NOTIFICATION_CONTENT_DATA_SAVER_ENABLED_TEXT_KEY) instanceof String value ? value : null;
			if (null == DataSaverEnabledText)
			{
				DataSaverEnabledText = "Data Saver Enabled";
			}

			WaitingForCellularText = data.get(DownloadWorkerParameterKeys.NOTIFICATION_CONTENT_WAITING_FOR_CELLULAR_TEXT_KEY) instanceof String value ? value : null;
			if (null == WaitingForCellularText)
			{
				WaitingForCellularText = "Waiting for cellular approval";
			}

			ApproveText = data.get(DownloadWorkerParameterKeys.NOTIFICATION_CONTENT_APPROVE_TEXT_KEY) instanceof String value ? value : null;
			if (null == ApproveText)
			{
				ApproveText = "Approve";
			}
		}
		
		//Load the Cancel Icon Resource
		{
			//Flag if the user tried to set anything so we can show an error if the load fails when they weren't expecting to use the default
			boolean bTriedToSetAnything = false;
			
			String CancelIconResourceName = data.get(DownloadWorkerParameterKeys.NOTIFICATION_RESOURCE_CANCEL_ICON_NAME) instanceof String value ? value : null;
			if (null == CancelIconResourceName) 
			{
				CancelIconResourceName = "ic_delete";
			}
			else
			{
				bTriedToSetAnything = true;
			}

			String CancelIconResourceType = data.get(DownloadWorkerParameterKeys.NOTIFICATION_RESOURCE_CANCEL_ICON_TYPE) instanceof String value ? value : null;
			if (null == CancelIconResourceType) 
			{
				CancelIconResourceType = "drawable";
			}
			else
			{
				bTriedToSetAnything = true;
			}

			String CancelIconResourcePackage = data.get(DownloadWorkerParameterKeys.NOTIFICATION_RESOURCE_CANCEL_ICON_PACKAGE) instanceof String value ? value : null;
			if (null == CancelIconResourcePackage) 
			{
				CancelIconResourcePackage = context.getPackageName();
			}
			else
			{
				bTriedToSetAnything = true;
			}

			CancelIconResourceID = context.getResources().getIdentifier(CancelIconResourceName, CancelIconResourceType, CancelIconResourcePackage);
			//Failed to load this default so try using a known default
			if (0 == CancelIconResourceID)
			{
				//Only an error if we set something and didn't expect the default for now
				if (bTriedToSetAnything)
				{
					if (null != Log) 
					{
						Log.error("Could not find resource for Cancel Icon using Name:" + CancelIconResourceName + " Type:" + CancelIconResourceType + " Package:" + CancelIconResourcePackage);
					}
				}
				
				//attempt to fallback to a system default
				CancelIconResourceID = android.R.drawable.ic_delete;
			
				if (0 == CancelIconResourceID)
				{
					if (null != Log)
					{
						Log.error("Unable to find any valid default cancel resource icon! Will likely crash from an invalid notification!");
					}
				}
			}
		}
		
		//Load the Small Icon Resource
		{
			//Flag if the user tried to set anything so we can show an error if the load fails when they weren't expecting to use the default
			boolean bTriedToSetAnything = false;
			
			String SmallIconResourceName = data.get(DownloadWorkerParameterKeys.NOTIFICATION_RESOURCE_SMALL_ICON_NAME) instanceof String value ? value : null;
			if (null == SmallIconResourceName)
			{
				SmallIconResourceName = "ic_notification_simple";
			}
			else
			{
				bTriedToSetAnything = true;
			}

			String SmallIconResourceType = data.get(DownloadWorkerParameterKeys.NOTIFICATION_RESOURCE_SMALL_ICON_TYPE) instanceof String value ? value : null;
			if (null == SmallIconResourceType)
			{
				SmallIconResourceType = "drawable";
			}
			else
			{
				bTriedToSetAnything = true;
			}

			String SmallIconResourcePackage = data.get(DownloadWorkerParameterKeys.NOTIFICATION_RESOURCE_SMALL_ICON_PACKAGE) instanceof String value ? value : null;
			if (null == SmallIconResourcePackage)
			{
				SmallIconResourcePackage = context.getPackageName();
			}
			else
			{
				bTriedToSetAnything = true;
			}

			SmallIconResourceID = context.getResources().getIdentifier(SmallIconResourceName, SmallIconResourceType, SmallIconResourcePackage);
			//Failed to load this default so try using the LocalNotification's paths
			if (0 == SmallIconResourceID)
			{
				if (bTriedToSetAnything)
				{
					if (null != Log) 
					{
						Log.error("Could not find resource for Small Icon using Name:" + SmallIconResourceName + " Type:" + SmallIconResourceType + " Package:" + SmallIconResourcePackage);
					}
				}
				
				//attempt to fallback to the LocalNotification's method
				SmallIconResourceID = LocalNotificationReceiver.getNotificationIconID(context);
			
				if (0 == SmallIconResourceID)
				{
					if (null != Log) 
					{
						Log.error("Could not find default resource for Small Icon! Will crash from invalid notification!");
					}
				}
			}
		}
	}

	//
	//Values set by owner
	//
	public int CurrentProgress = 0;
	public boolean Indeterminate = true;
		
	//
	//Values we load defaults into from the WorkerParameters
	//
	public String NotificationChannelID = null;
	public String NotificationChannelName = null;
	public int NotificationChannelImportance = NotificationManager.IMPORTANCE_DEFAULT;
	
	public int NotificationID = 0;

	public boolean ShouldHandleCellular = false;

	public boolean ShouldShowPercentage = false;

	public String TitleText = null;
	public String ContentText = null;
	public String ContentCompleteText = null;
	public String CancelText = null;
	public String NoInternetAvailable = null;
	public String AirplaneModeText = null;
	public String DataSaverEnabledText = null;
	public String WaitingForCellularText = null;
	public String ApproveText = null;
	
	public int CancelIconResourceID = 0;
	public int SmallIconResourceID = 0;

	//We just care about our progress being between 0->100 as a % so this is always 100
	public final int MAX_PROGRESS = 100;
}