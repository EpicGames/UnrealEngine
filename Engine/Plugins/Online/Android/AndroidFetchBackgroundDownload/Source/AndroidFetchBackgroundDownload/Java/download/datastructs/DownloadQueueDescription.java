// Copyright Epic Games, Inc. All Rights Reserved.
package com.epicgames.unreal.download.datastructs;

import android.content.Context;
import android.content.SharedPreferences;

import androidx.annotation.NonNull;
import androidx.annotation.Nullable;
import androidx.work.Data;

import com.epicgames.unreal.Logger;
import com.epicgames.unreal.download.DownloadProgressListener;

import org.json.JSONArray;
import org.json.JSONException;

import java.util.ArrayList;
import java.util.Map;


//Helper class that is used to serialize any information needed for our downloads as a whole from/to WorkerParameters
public class DownloadQueueDescription
{
	public final static int DEFAULT_MAX_CONCURRENT_DOWNLOADS = 4;

	public DownloadQueueDescription(ArrayList<DownloadDescription> downloadDescriptions, @NonNull Map<String, ?> data)
	{
		DownloadDescriptions = downloadDescriptions;
		MaxConcurrentDownloads = data.get(DownloadWorkerParameterKeys.DOWNLOAD_MAX_CONCURRENT_REQUESTS_KEY) instanceof Integer value ? value : DEFAULT_MAX_CONCURRENT_DOWNLOADS;
	}

	public DownloadQueueDescription(ArrayList<DownloadDescription> downloadDescriptions, @NonNull SharedPreferences backgroundPreferences)
	{
		this(downloadDescriptions, backgroundPreferences.getAll());
	}

	//Constructor that parses the List<DownloadDescription> from our Worker InputData (WorkerParameters)
	public DownloadQueueDescription(@NonNull String DownloadDescriptionListFileName, @NonNull Data data, @Nullable Logger Log)
	{
		this(ParseDownloadDescriptionString(DownloadDescriptionListFileName, Log), data.getKeyValueMap());
	}

	public DownloadQueueDescription(@NonNull String DownloadDescriptionListFileName, @NonNull Data data, @NonNull Context context, @Nullable Logger Log)
	{
		this(DownloadDescriptionListFileName, data, Log);
	}

	public static String GetDownloadDescriptionListFileName(@NonNull Data data, @Nullable Logger Log)
	{
		//Parse DownloadDescriptions 
		String DownloadDescriptionListString = data.getString(DownloadWorkerParameterKeys.DOWNLOAD_DESCRIPTION_LIST_KEY);
		if (null == DownloadDescriptionListString)
		{
			if (null != Log)
			{
				Log.error(DownloadWorkerParameterKeys.NOTIFICATION_CHANNEL_ID_KEY + " key returned null list! No downloads to process in WorkerParameters!");
			}
		}

		return DownloadDescriptionListString;
	}

	//Parses our DownloadDescriptions member based on the passed in JSONObject file
	static ArrayList<DownloadDescription> ParseDownloadDescriptionString(@NonNull String DownloadDescriptionListStringFileName, @Nullable Logger Log)
	{
		ArrayList<DownloadDescription> DownloadDescriptions = new ArrayList<>();
		
		try 
		{
			String JSONInFile = DownloadDescription.GetDownloadDescriptionJSONArrayFromFile(DownloadDescriptionListStringFileName);
			
			if (null != JSONInFile) 
			{
				JSONArray DescriptionJsonArray = new JSONArray(JSONInFile);

				for (int JsonIndex = 0; JsonIndex < DescriptionJsonArray.length(); ++JsonIndex) 
				{
					String DescriptionJsonString = DescriptionJsonArray.optString(JsonIndex);
					DownloadDescription ParsedDescription = DownloadDescription.FromJSON(DescriptionJsonString);
					DownloadDescriptions.add(ParsedDescription);
				}
			}
			else
			{
				Log.error("Failure loading JSON from file " + DownloadDescriptionListStringFileName);
			}
		}
		catch (JSONException e) 
		{
			e.printStackTrace();
			
			if (null != Log)
			{
				Log.error("Exception while parsing download descriptions: " + e.getMessage());
			}			
		}
		
		return DownloadDescriptions;
	}

	public ArrayList<DownloadDescription> DownloadDescriptions;
	public DownloadProgressListener ProgressListener = null;
	public int DownloadGroupID = 0;
	public int MaxConcurrentDownloads = DEFAULT_MAX_CONCURRENT_DOWNLOADS;
}