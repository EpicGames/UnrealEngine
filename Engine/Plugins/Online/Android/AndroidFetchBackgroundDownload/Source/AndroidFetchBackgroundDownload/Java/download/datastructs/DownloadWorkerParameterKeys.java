// Copyright Epic Games, Inc. All Rights Reserved.
package com.epicgames.unreal.download.datastructs;

//
// WARNING: These keys values must stay in sync with their values in FAndroidNativeDownloadWorkerParameterKeys!
//

//Class wrapper just holding static information about our different WorkerParameter key names for storing/retrieving values from the WorkerParameters.
public class DownloadWorkerParameterKeys
{
	//
	// Keys for general worker request settings
	//
	//
	// Keys for Download parsing
	//
	public final static String DOWNLOAD_DESCRIPTION_LIST_KEY = "DownloadDescriptionList";
	public final static String DOWNLOAD_MAX_CONCURRENT_REQUESTS_KEY = "MaxConcurrentDownloadRequests";
	
	//public final static String DOWNLOAD_GROUP_ID_KEY = "DownloadGroupID";
	//
	// Keys for our notification displayed if setForegroundAsync is called
	//
	//TODO: TRoss These keys should be moved up into a UEWorkerParameterKeys that this class extends.
	public final static String NOTIFICATION_CHANNEL_ID_KEY = "NotificationChannelId";
	public final static String NOTIFICATION_CHANNEL_NAME_KEY = "NotificationChannelName";
	public final static String NOTIFICATION_CHANNEL_IMPORTANCE_KEY = "NotificationChannelImportance";
	
	public final static String NOTIFICATION_ID_KEY = "NotificationId";
	//random value that our NOTIFICATION_ID is set to if not provided using the above key
	public final static int NOTIFICATION_DEFAULT_ID_KEY = 1923901283;

	public final static String CELLULAR_HANDLE_KEY = "CellularHandleKey";

	public final static String SHOW_PERCENTAGE_KEY = "bShowPercentage";
	
	public final static String NOTIFICATION_CONTENT_TITLE_KEY = "NotificationContentTitle";
	public final static String NOTIFICATION_CONTENT_TEXT_KEY = "NotificationContentText";
	public final static String NOTIFICATION_CONTENT_CANCEL_DOWNLOAD_TEXT_KEY = "NotificationContentCancelDownloadText";
	public final static String NOTIFICATION_CONTENT_NO_INTERNET_TEXT_KEY = "NotificationContentNoInternetDownloadText";
	public final static String NOTIFICATION_CONTENT_AIRPLANE_MODE_TEXT_KEY = "NotificationContentAirplaneModeText";
	public final static String NOTIFICATION_CONTENT_DATA_SAVER_ENABLED_TEXT_KEY = "NotificationContentDataSaverEnabledText";
	public final static String NOTIFICATION_CONTENT_COMPLETE_TEXT_KEY = "NotificationContentCompleteText";
	public final static String NOTIFICATION_CONTENT_WAITING_FOR_CELLULAR_TEXT_KEY = "NotificationContentWaitingForCellularText";
	public final static String NOTIFICATION_CONTENT_APPROVE_TEXT_KEY = "NotificationContentApproveText";

	public final static String NOTIFICATION_RESOURCE_CANCEL_ICON_NAME = "NotificationResourceCancelIconName";
	public final static String NOTIFICATION_RESOURCE_CANCEL_ICON_TYPE = "NotificationResourceCancelIconType";
	public final static String NOTIFICATION_RESOURCE_CANCEL_ICON_PACKAGE = "NotificationResourceCancelIconPackage";

	public final static String NOTIFICATION_RESOURCE_SMALL_ICON_NAME = "NotificationResourceSmallIconName";
	public final static String NOTIFICATION_RESOURCE_SMALL_ICON_TYPE = "NotificationResourceSmallIconType";
	public final static String NOTIFICATION_RESOURCE_SMALL_ICON_PACKAGE = "NotificationResourceSmallIconPackage";
}