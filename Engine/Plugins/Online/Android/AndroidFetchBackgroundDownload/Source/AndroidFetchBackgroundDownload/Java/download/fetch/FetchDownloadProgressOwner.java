// Copyright Epic Games, Inc. All Rights Reserved.
package com.epicgames.unreal.download.fetch;

import androidx.annotation.NonNull;

import com.epicgames.unreal.download.DownloadProgressListener;
import com.epicgames.unreal.download.fetch.FetchRequestProgressListener;

import com.tonyodev.fetch2.Download;
import com.tonyodev.fetch2.Error;
import com.tonyodev.fetch2.FetchGroup;

//Interface between the FetchListener and the FetchRequestProgressListener. Receives calls routed through the FetchRequestProgressListener from the Fetch2 instance.
public interface FetchDownloadProgressOwner
{
	public void OnDownloadQueued(@NonNull Download download);
	public void OnDownloadStarted(@NonNull Download download);
	public void OnDownloadProgress(@NonNull Download download, boolean indeterminate, long downloadedBytesPerSecond, long etaInMilliSeconds);
	public void OnDownloadChangePauseState(@NonNull Download download, boolean bIsPaused);
	public void OnDownloadGroupProgress(@NonNull FetchGroup Group, DownloadProgressListener ProgressListener);
	
	public void OnDownloadComplete(@NonNull Download download, FetchRequestProgressListener.ECompleteReason completeReason);
}