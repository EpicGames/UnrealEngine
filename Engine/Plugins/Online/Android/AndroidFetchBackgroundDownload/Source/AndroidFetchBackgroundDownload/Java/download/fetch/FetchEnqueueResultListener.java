// Copyright Epic Games, Inc. All Rights Reserved.
package com.epicgames.unreal.download.fetch;

import androidx.annotation.NonNull;

import com.tonyodev.fetch2.Error;
import com.tonyodev.fetch2.Request;
import com.tonyodev.fetch2core.Func;

//Interface describing a class that wants to receive results from a Fetch2 enqueue call
public interface FetchEnqueueResultListener
{
	//Function called on this FetchEnqueueResultListener by the FetchEnqueueRequestCallback
	public void OnFetchEnqueueRequestCallback(@NonNull String DestinationLocation, @NonNull Request EnqueuedRequest);
	
	//Function called on this FetchEnqueueResultListener by the FetchEnqueueErrorCallback
	public void OnFetchEnqueueErrorCallback(@NonNull String DestinationLocation, @NonNull Error EnqueueError);
	
	//Helper class passed in to the Fetch.enqueue command to pass back the results of the enqueue operation on success.
	public class FetchEnqueueRequestCallback implements Func<Request>
	{
		public FetchEnqueueRequestCallback(@NonNull FetchEnqueueResultListener Owner, @NonNull String DestinationLocation)
		{
			this.Owner = Owner;
			this.DestinationLocation = DestinationLocation;
		}

		@Override
		public void call(@NonNull Request result)
		{
			if (Owner != null) 
			{
				Owner.OnFetchEnqueueRequestCallback(DestinationLocation, result);
			}			
			Owner = null;
		}

		FetchEnqueueResultListener Owner = null;
		String DestinationLocation = null;
	}

	//Helper class passed in to the Fetch.enqueue command to pass back the results of the enqueue operation on error
	public class FetchEnqueueErrorCallback implements Func<Error>
	{
		public FetchEnqueueErrorCallback(@NonNull FetchEnqueueResultListener Owner, @NonNull String DestinationLocation)
		{
			this.Owner = Owner;
			this.DestinationLocation = DestinationLocation;
		}

		@Override
		public void call(@NonNull Error error)
		{
			if (Owner != null) 
			{
				Owner.OnFetchEnqueueErrorCallback(DestinationLocation, error);
			}
			Owner = null;
		}
		
		FetchEnqueueResultListener Owner = null;
		String DestinationLocation = null;
	}
}