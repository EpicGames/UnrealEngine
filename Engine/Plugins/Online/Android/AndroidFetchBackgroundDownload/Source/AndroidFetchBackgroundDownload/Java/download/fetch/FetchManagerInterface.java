// Copyright Epic Games, Inc. All Rights Reserved.

package com.epicgames.unreal.download.fetch;

import android.content.Context;
import androidx.annotation.NonNull;

import com.epicgames.unreal.download.datastructs.DownloadQueueDescription;
import com.epicgames.unreal.download.DownloadProgressListener;


public interface FetchManagerInterface
{
	public interface AllDownloadsCompletedListener
	{
		public void OnAllDownloadsCompleted(boolean bDidAllSucceed);
	}

	public void StopWork(String WorkID);
	public void EnqueueRequests(@NonNull Context context, @NonNull DownloadQueueDescription QueueDescription);
	public void PauseDownload(String DestinationLocation, boolean bPause);
	public void PauseAllDownloads();
	public void ResumeAllDownloads();
	public void CancelDownload(String DestinationLocation);
	public void RetryDownload(String DestinationLocation);
	public void RequestGroupProgressUpdate(int GroupID, DownloadProgressListener ListenerToUpdate);
	public void SetAllDownloadsCompletedListener(@NonNull AllDownloadsCompletedListener Listener);
	default public void NetworkAvailable() {}
	default public void NetworkLost() {}
}
