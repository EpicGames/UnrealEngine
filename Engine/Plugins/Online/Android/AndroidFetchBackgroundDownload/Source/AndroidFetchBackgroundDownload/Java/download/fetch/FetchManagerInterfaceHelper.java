// Copyright Epic Games, Inc. All Rights Reserved.

package com.epicgames.unreal.download.fetch;

import com.epicgames.unreal.Logger;
import com.epicgames.unreal.GameActivity;

import android.content.Context;
import android.content.SharedPreferences;

import java.io.File;

public class FetchManagerInterfaceHelper {

	public final static String NEW_FETCH_IMPLEMENATION_KEY = "UseNewFetchImplementation";
	public final static String TEMP_DOWNLOAD_ROOT_PATH_KEY = "TempDownloadRootPath";

	public static void InitSharedManager(Context context)
	{
		if (SharedManager == null) {
			final SharedPreferences preferences = context.getSharedPreferences("BackgroundPreferences", context.MODE_PRIVATE);

			// cleanup all temp content not in a directory named the same as our build version
			String tempDownloadRootPath = null;
			try {
				tempDownloadRootPath = preferences.getString(TEMP_DOWNLOAD_ROOT_PATH_KEY, null);
			} catch (ClassCastException notPresent) {}
			if (tempDownloadRootPath != null && !tempDownloadRootPath.isEmpty()) {
				File[] files = new File(tempDownloadRootPath).listFiles();
				if (files != null) {
					int deletedCount = 0;
					for (File file : files) {
						if (!file.getName().endsWith("DownloadDescriptionJSONs") &&
							!file.getName().endsWith("DownloadDescriptions") &&
							!file.getName().endsWith("URLMap") &&
							!file.getName().equals(GameActivity.BGDL_BUILD_VERSION)) {
							Log.debug("cleaning up: " + file.getName());
							DeleteRecursive(file);
							++deletedCount;
						}
					}
					Log.debug("InitSharedManager: removed " + deletedCount + " files/directories from temp download directory: " + tempDownloadRootPath);
				} else {
					Log.debug("InitSharedManager: temp download directory did not exist: " + tempDownloadRootPath);
				}
			} else {
				Log.debug("InitSharedManager: no known temp download directory to clean up");
			}

			// then decide what implementation to use, new or old, and instantiate it
			boolean bUseNewFetchImplementation = false;
			try {
				bUseNewFetchImplementation = preferences.getBoolean(NEW_FETCH_IMPLEMENATION_KEY, false);
			} catch (ClassCastException notPresent) {}
			if (bUseNewFetchImplementation) {
				Log.debug("Using new fetch manager implementation");
				SharedManager = FetchManagerNew.GetSharedManager();
			} else {
				Log.debug("Using old fetch manager implementation");
				SharedManager = FetchManagerOld.GetSharedManager();
			}
		} else {
			Log.debug("InitSharedManager: ignoring subsequent calls");
		}
	}

	public static FetchManagerInterface GetSharedManager()
	{
		if (SharedManager == null) {
			Log.debug("GetSharedManager() called too early, InitSharedManager not yet called!");
			SharedManager = FetchManagerOld.GetSharedManager();
		}
		return SharedManager;
	}

	private static Logger Log = new Logger("UE", "FetchManagerInterfaceHelper");
	private static FetchManagerInterface SharedManager = null;

	public static boolean DeleteRecursive(File fileOrDirectory) {
		if (fileOrDirectory.isDirectory()) {
			for (File child : fileOrDirectory.listFiles()) {
				DeleteRecursive(child);
			}
		}
		return fileOrDirectory.delete();
	}
}
