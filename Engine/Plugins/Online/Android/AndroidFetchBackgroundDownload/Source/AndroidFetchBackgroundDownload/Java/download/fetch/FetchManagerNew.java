// Copyright Epic Games, Inc. All Rights Reserved.

package com.epicgames.unreal.download.fetch;

import android.content.Context;
import android.content.SharedPreferences;

import androidx.annotation.NonNull;

import com.epicgames.unreal.Logger;
import com.epicgames.unreal.download.DownloadProgressListener;
import com.epicgames.unreal.download.UEDownloadWorker.EDownloadCompleteReason;
import com.epicgames.unreal.download.asyncdownloader.AsyncDownloadRequest;
import com.epicgames.unreal.download.asyncdownloader.AsyncDownloadRequestListener;
import com.epicgames.unreal.download.asyncdownloader.AsyncDownloader;
import com.epicgames.unreal.download.asyncdownloader.AsyncDownloaderListener;
import com.epicgames.unreal.download.datastructs.DownloadDescription;
import com.epicgames.unreal.download.datastructs.DownloadQueueDescription;

import java.io.File;
import java.net.URL;
import java.util.List;

public class FetchManagerNew implements FetchManagerInterface
{
	// ---------------------------------------------------------------------------------------------------------------------
	// public interface
	// ---------------------------------------------------------------------------------------------------------------------	

	public static synchronized FetchManagerNew GetSharedManager()
	{
		return SharedManager;
	}
	
	public Object SyncRoot()
	{
		return AsyncDownloader;
	}

	@Override	
	public synchronized void StopWork(String WorkID)
	{
		Log.debug("FetchManagerNew: StopWork");
		if (AsyncDownloader != null)
		{
			AsyncDownloader.CancelAll();
		}
	}

	public synchronized void Enqueue(@NonNull Context context, @NonNull DownloadQueueDescription QueueDescription)
	{
		if (AsyncDownloader == null)
		{
			SharedPreferences preferences = context.getSharedPreferences("BackgroundDownload", Context.MODE_PRIVATE);
			boolean AllowHTTP2 = preferences.getBoolean("AllowHTTP2", false);
			AsyncDownloader = new AsyncDownloader(AllowHTTP2, QueueDescription.MaxConcurrentDownloads);
			SetupAllDownloadsCompletedListener_Internal();
		}
		else
		{
			AsyncDownloader.SetMaxConcurrent(QueueDescription.MaxConcurrentDownloads);
		}

		GlobalDownloadProgressListener = QueueDescription.ProgressListener;

		for (DownloadDescription DownloadDescription : QueueDescription.DownloadDescriptions)
		{
			//noinspection SynchronizeOnNonFinalField
			synchronized (AsyncDownloader)
			{
				final String TargetLocation = DownloadDescription.DestinationLocation;
				final String DownloadLocation = DownloadDescription.DestinationLocation + ".temp";
				final boolean bFileExists = new File(TargetLocation).exists();

				if (bFileExists)
				{
					GlobalDownloadProgressListener.OnDownloadMetrics(DownloadDescription.DestinationLocation, new File(TargetLocation).length(), 0, DownloadDescription.DownloadStartTime, DownloadDescription.DownloadStartTime);
					GlobalDownloadProgressListener.OnDownloadComplete(DownloadDescription.DestinationLocation, EDownloadCompleteReason.Success);

					AsyncDownloader.Cancel(DownloadDescription.DestinationLocation);
					continue;
				}

				final boolean bSuccess = AsyncDownloader.Submit(
					new AsyncDownloadRequest.Builder()
						.SetId(DownloadDescription.DestinationLocation)
						.SetRequestId(DownloadDescription.RequestID)
						.SetUrls(DownloadDescription.URLs)
						.SetMaxRetries(DownloadDescription.MaxRetryCount)
						.SetGroupID(DownloadDescription.GroupID)
						.SetTargetFile(new File(DownloadLocation))
						.SetResumeIfPossible(true)
						.SetTotalSize(DownloadDescription.TotalBytesNeeded)
						.Build(),
					new AsyncDownloadRequestListenerImpl(this, GlobalDownloadProgressListener, TargetLocation, DownloadLocation));
				if (!bSuccess)
				{
					RetryDownload(DownloadDescription.DestinationLocation);
				}

				GlobalDownloadProgressListener.OnDownloadEnqueued(DownloadDescription);
			}
		}
	}
	
	@Override
	public synchronized void EnqueueRequests(@NonNull Context context, @NonNull DownloadQueueDescription QueueDescription)
	{
		Log.debug("FetchManagerNew: EnqueueRequests with " + QueueDescription.DownloadDescriptions.size() + " requests");

		if (AsyncDownloader != null)
		{
			AsyncDownloader.CancelAll();
		}

		NumFailedRequests = 0;
		if (QueueDescription.DownloadDescriptions.isEmpty()) {
			return;
		}

		if (AsyncDownloader == null)
		{
			SharedPreferences preferences = context.getSharedPreferences("BackgroundDownload", Context.MODE_PRIVATE);
			boolean AllowHTTP2 = preferences.getBoolean("AllowHTTP2", false);
			AsyncDownloader = new AsyncDownloader(AllowHTTP2, QueueDescription.MaxConcurrentDownloads);
			SetupAllDownloadsCompletedListener_Internal();
		}
		else
		{
			AsyncDownloader.SetMaxConcurrent(QueueDescription.MaxConcurrentDownloads);
		}

		GlobalDownloadProgressListener = QueueDescription.ProgressListener;

		for (DownloadDescription DownloadDescription : QueueDescription.DownloadDescriptions)
		{
			//noinspection SynchronizeOnNonFinalField
			synchronized (AsyncDownloader)
			{
				final String TargetLocation = DownloadDescription.DestinationLocation;
				final String DownloadLocation = DownloadDescription.DestinationLocation + ".temp";
				final boolean bFileExists = new File(TargetLocation).exists();

				if (bFileExists)
				{
					GlobalDownloadProgressListener.OnDownloadMetrics(DownloadDescription.DestinationLocation, new File(TargetLocation).length(), 0, DownloadDescription.DownloadStartTime, DownloadDescription.DownloadStartTime);
					GlobalDownloadProgressListener.OnDownloadComplete(DownloadDescription.DestinationLocation, EDownloadCompleteReason.Success);
					
					AsyncDownloader.Cancel(DownloadDescription.DestinationLocation);
					continue;
				}
				
				final boolean bSuccess = AsyncDownloader.Submit(
					new AsyncDownloadRequest.Builder()
						.SetId(DownloadDescription.DestinationLocation)
						.SetRequestId(DownloadDescription.RequestID)
						.SetUrls(DownloadDescription.URLs)
						.SetMaxRetries(DownloadDescription.MaxRetryCount)
						.SetGroupID(DownloadDescription.GroupID)
						.SetTargetFile(new File(DownloadLocation))
						.SetResumeIfPossible(true)
						.SetTotalSize(DownloadDescription.TotalBytesNeeded)
						.Build(),
					new AsyncDownloadRequestListenerImpl(this, GlobalDownloadProgressListener, TargetLocation, DownloadLocation));
				if (!bSuccess)
				{
					RetryDownload(DownloadDescription.DestinationLocation);
				}

				GlobalDownloadProgressListener.OnDownloadEnqueued(DownloadDescription);
			}
		}

		// clear flag that stops downloader from downloading
		ResumeAllDownloads();
	}

	@Override
	public synchronized void PauseDownload(String DestinationLocation, boolean bPause)
	{
		if (AsyncDownloader == null) {
			return;
		}
		if (bPause) {
			AsyncDownloader.Pause(DestinationLocation, true);
		} else {
			AsyncDownloader.Resume(DestinationLocation, true);
		}
	}

	@Override
	public synchronized void PauseAllDownloads()
	{
		if (AsyncDownloader == null) {
			return;
		}
		AsyncDownloader.PauseAll();
	}

	@Override
	public synchronized void ResumeAllDownloads()
	{
		if (AsyncDownloader == null) {
			return;
		}
		AsyncDownloader.ResumeAll();
	}

	@Override
	public synchronized void CancelDownload(String DestinationLocation)
	{
		if (AsyncDownloader == null) {
			return;
		}
		AsyncDownloader.Cancel(DestinationLocation);
	}

	@Override
	public synchronized void RetryDownload(String DestinationLocation)
	{
		if (AsyncDownloader == null) {
			return;
		}
		AsyncDownloader.Retry(DestinationLocation);
	}

	public synchronized String[] GetRunningDestinationLocations()
	{
		if (AsyncDownloader == null) {
			return new String[0];
		}
		return AsyncDownloader.GetRunningDestinationLocations();
	}

	@Override
	public synchronized void RequestGroupProgressUpdate(int GroupID, DownloadProgressListener ListenerToUpdate)
	{
		if (GlobalDownloadProgressListener == null) {
			return;
		}

		if (AsyncDownloader == null) {
			GlobalDownloadProgressListener.OnDownloadGroupProgress(GroupID, 0, true);
			return;
		}

		float Progress = AsyncDownloader.GetGroupProgress(GroupID);
		if (Progress < 0) {
			GlobalDownloadProgressListener.OnDownloadGroupProgress(GroupID, 0, true);
			return;
		}

		GlobalDownloadProgressListener.OnDownloadGroupProgress(GroupID,  (int)Math.ceil(Progress * 100.0f), false);
	}
	
	@Override
	public synchronized void SetAllDownloadsCompletedListener(@NonNull AllDownloadsCompletedListener Listener)
	{
		AllDownloadsCompletedListener = Listener;
		SetupAllDownloadsCompletedListener_Internal();
	}

	@Override
	public void NetworkAvailable() 
	{
		if (AsyncDownloader == null) {
			return;
		}
		AsyncDownloader.ResumeAll();
	}

	@Override
	public void NetworkLost() {
		if (AsyncDownloader == null) {
			return;
		}
		AsyncDownloader.PauseAll();
	}

	// ---------------------------------------------------------------------------------------------------------------------
	// private data + methods
	// ---------------------------------------------------------------------------------------------------------------------

	private FetchManagerNew() {}

	private void SetupAllDownloadsCompletedListener_Internal() {
		if (AsyncDownloader != null) {
			AsyncDownloader.SetAsyncDownloaderListener(new AsyncDownloaderListener() {
				@Override
				public void OnAllCompleted() {
					final boolean bAllRequestsSucceeded = NumFailedRequests == 0;

					if (AllDownloadsCompletedListener != null) {
						AllDownloadsCompletedListener.OnAllDownloadsCompleted(bAllRequestsSucceeded);
					}

					if (GlobalDownloadProgressListener != null) {
						GlobalDownloadProgressListener.OnAllDownloadsComplete(bAllRequestsSucceeded);
					}
				}
			});
		}
	}
	
	private static final FetchManagerNew SharedManager = new FetchManagerNew();
	private volatile AllDownloadsCompletedListener AllDownloadsCompletedListener = null;
	private volatile AsyncDownloader AsyncDownloader = null;
	private volatile DownloadProgressListener GlobalDownloadProgressListener = null;
	private volatile int NumFailedRequests = 0;
	private final Logger Log = new Logger("UE", "FetchManagerNew");

	// ---------------------------------------------------------------------------------------------------------------------
	// private helper class
	// ---------------------------------------------------------------------------------------------------------------------

	static final private class AsyncDownloadRequestListenerImpl implements AsyncDownloadRequestListener {

		public AsyncDownloadRequestListenerImpl(@NonNull FetchManagerNew Owner, @NonNull DownloadProgressListener ProgressListener, @NonNull String TargetLocation, @NonNull String DownloadLocation) {
			this.Owner = Owner;
			this.ProgressListener = ProgressListener;
			this.TargetLocation = TargetLocation;
			this.DownloadLocation = DownloadLocation;
		}

		@Override
		public void OnStart(@NonNull String Id, @NonNull String DebugString, long StartTimeUTC) {
			StartTime = StartTimeUTC;
			ProgressListener.OnDownloadStart(Id, DebugString, StartTimeUTC);
		}
		@Override
		public void OnProgress(@NonNull String Id, long bytesDownloadedSoFar) {
			// Log.debug("OnProgress: " + Id + " | " + bytesDownloadedSoFar + " | " + totalBytesOrMinus1);
			LastDownloadedSoFar = bytesDownloadedSoFar;

			long NewUpdateTime = System.currentTimeMillis();
			if (NewUpdateTime >= UpdateTime + 100) {
				long TotalDownloadedSinceLastUpdate = LastDownloadedSoFar - LastUpdatedSoFar;
				LastUpdatedSoFar = LastDownloadedSoFar;
				UpdateTime = NewUpdateTime;
				ProgressListener.OnDownloadProgress(Id, LastUpdatedSoFar, TotalDownloadedSinceLastUpdate);
			}
		}
		@Override
		public void OnSuccess(@NonNull String Id, @NonNull File file) {
			EDownloadCompleteReason Reason = EDownloadCompleteReason.Success;
			File NewFileAtDestinationLocation = new File(TargetLocation);
			if (!file.renameTo(NewFileAtDestinationLocation)) {
				Log.error("OnSuccess: Failed to rename " + DownloadLocation + " to " + TargetLocation + ", switching to error instead");
				Reason = EDownloadCompleteReason.Error;
			}

			OnCompleteDownload_Internal(Id, Reason);
		}
		@Override
		public void OnError(@NonNull String Id, @NonNull Throwable Error, URL LastTriedUrl) {
			Log.debug("OnError: " + Id);
			OnCompleteDownload_Internal(Id, EDownloadCompleteReason.OutOfRetries);
		}
		@Override
		public void OnPaused(@NonNull String Id) {
			// do we need to do something here?
		}

		private void OnCompleteDownload_Internal(@NonNull String Id, EDownloadCompleteReason Reason) {
			long EndTime = System.currentTimeMillis();

			long Duration = 0;
			if (EndTime > StartTime) {
				Duration = EndTime - StartTime;
			}

			if (Reason != EDownloadCompleteReason.Success) {
				Owner.NumFailedRequests++;
			}

			long TotalDownloadedSinceLastUpdate = LastDownloadedSoFar - LastUpdatedSoFar;
			if (TotalDownloadedSinceLastUpdate > 0)	{
				LastUpdatedSoFar = LastDownloadedSoFar;
				UpdateTime = EndTime;
				ProgressListener.OnDownloadProgress(Id, LastUpdatedSoFar, TotalDownloadedSinceLastUpdate);
			}
			ProgressListener.OnDownloadMetrics(Id, LastUpdatedSoFar, Duration, StartTime, EndTime);
			ProgressListener.OnDownloadComplete(Id, Reason);
		}

		private long LastDownloadedSoFar = 0;
		private long StartTime = -1;
		private long LastUpdatedSoFar = 0;
		private long UpdateTime = Long.MIN_VALUE;
		private final @NonNull DownloadProgressListener ProgressListener;
		private final @NonNull FetchManagerNew Owner;
		private final Logger Log = new Logger("UE", "FetchManagerNew");
		private final String TargetLocation;
		private final String DownloadLocation;
	}
}
