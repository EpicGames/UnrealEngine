// Copyright Epic Games, Inc. All Rights Reserved.
package com.epicgames.unreal.download.fetch;

import android.content.Context;
import android.content.SharedPreferences;
import android.net.Uri;

import com.epicgames.unreal.Logger;
import com.epicgames.unreal.download.datastructs.DownloadDescription;
import com.epicgames.unreal.download.datastructs.DownloadQueueDescription;
import com.epicgames.unreal.download.fetch.FetchDownloadProgressOwner;
import com.epicgames.unreal.download.DownloadProgressListener;
import com.epicgames.unreal.download.fetch.FetchEnqueueResultListener;
import com.epicgames.unreal.download.fetch.FetchRequestProgressListener;

import com.epicgames.unreal.download.UEDownloadWorker.EDownloadCompleteReason;
import com.epicgames.unreal.download.fetch.FetchRequestProgressListener.ECompleteReason;

import com.tonyodev.fetch2.CompletedDownload;
import com.tonyodev.fetch2.Download;
import com.tonyodev.fetch2.Error;
import com.tonyodev.fetch2.Fetch;
import com.tonyodev.fetch2.FetchConfiguration;
import com.tonyodev.fetch2.FetchGroup;
import com.tonyodev.fetch2.FetchListener;
import com.tonyodev.fetch2.NetworkType;
import com.tonyodev.fetch2.Priority;
import com.tonyodev.fetch2.Request;
import com.tonyodev.fetch2.exception.FetchException;
import com.tonyodev.fetch2core.FetchLogger;
import com.tonyodev.fetch2core.Func;
import com.tonyodev.fetch2core.Func2;
import com.tonyodev.fetch2.Status;

import com.tonyodev.fetch2okhttp.OkHttpDownloader;
import okhttp3.OkHttpClient;

import androidx.annotation.Nullable;
import androidx.annotation.NonNull;

import java.util.ArrayList;
import java.util.HashMap;
import java.util.List;
import java.util.Set;
import java.util.Map;

import java.io.File;

//Class that handles setting up and managing Fetch2 requests
public class FetchManagerOld implements FetchDownloadProgressOwner, FetchEnqueueResultListener, FetchManagerInterface
{
	private enum ETaskRunningState
	{
		Unknown,
		Running,
		Paused
	};
	
	private FetchManagerOld()
	{
		RequestedDownloads =  new HashMap<String,DownloadDescription>();
	}
	
	public static synchronized FetchManagerOld GetSharedManager()
	{
		return SharedManager;
	}
	
	@Override
	public synchronized void StopWork(String WorkID)
	{
		Log.debug("StopWork called");
		runningState = ETaskRunningState.Unknown;

		//Immediately remove FetchListener to prevent us from getting late callbacks after work has stopped
		if (FetchListener != null)
		{
			if (IsFetchInstanceValid())
			{
				FetchInstance.removeListener(FetchListener);
			}
			FetchListener = null;
		}

		//Synchronized with initFetch to make sure we aren't creating and closing our fetch instance in a weird race condition
		if (IsFetchInstanceValid())
		{
			//We need to purge all partially completed downloads
			//If we don't, then on relaunching our app the BackgroundHttp system can end up assuming these downloads are finished incorrectly.
			DeleteAllInProgressRequests();

			//close our FetchInstance so it stops all work until we recreate it in initfetch later
			FetchInstance.close();
			FetchInstance = null;
		}
	}
	
	private synchronized void DeleteAllInProgressRequests()
	{
		runningState = ETaskRunningState.Unknown;
		if (!IsFetchInstanceValid())
		{
			Log.error("Call to DeleteAllInProgressRequests with an unexpected invalid FetchInstance");
			return;
		}

		//First we remove from Fetch management any completed downloads, this is so they aren't deleted by the next call
		FetchInstance.removeAllWithStatus(Status.COMPLETED);
			
		//Deletes everything still being managed by the FetchInstance
		FetchInstance.deleteAll();
	}

	@Override
	public synchronized void EnqueueRequests(Context context, DownloadQueueDescription QueueDescription)
	{
		runningState = ETaskRunningState.Unknown;
		InitFetch(context, QueueDescription.MaxConcurrentDownloads);
		
		int NumDownloads = QueueDescription.DownloadDescriptions.size();
		Log.debug("EnqueueRequests called with " + NumDownloads + " DownloadDescriptions. Current RequestedDownloads Size: " + RequestedDownloads.size());

		SetVariablesFromQueueDescription(QueueDescription);
		ReconcileDownloadDescriptions(QueueDescription);
	}
		
	//Does any setup we need based on data in the QueueDescription
	public synchronized void SetVariablesFromQueueDescription(DownloadQueueDescription QueueDescription)
	{
		if (!IsFetchInstanceValid())
		{
			Log.error("Call to SetVariablesFromQueueDescription after FetchInstance is invalidated!");
			return;
		}
		
		FetchInstance.setDownloadConcurrentLimit(QueueDescription.MaxConcurrentDownloads);		
	}
		
	//Goes through the DownloadDescriptions in our DownloadQueueDescription and finds DownloadDescription that
	//don't match our ActiveDownloadDescriptions entries and thus need to be reconciled (either added or modified in some way)
	private synchronized void ReconcileDownloadDescriptions(DownloadQueueDescription QueueDescription)
	{
		//Create reference copy of RequestedDownloads and clear original lists for
		//RequestedDownloads, CompletedDownloads, and FailedDownloads to allow them to re-populate with new work state
		HashMap<String, DownloadDescription> TempRequestedDownloadsCopy = RequestedDownloads;
		RequestedDownloads = new HashMap<String, DownloadDescription>();
		CompletedDownloads.clear();
		FailedDownloads.clear();

		final int NumNewDownloadDescriptions = QueueDescription.DownloadDescriptions.size();
		for (int DescriptionIndex = 0; DescriptionIndex < NumNewDownloadDescriptions; ++DescriptionIndex)
		{
			DownloadDescription NewDownloadDescription = QueueDescription.DownloadDescriptions.get(DescriptionIndex);
			
			//Set any DownloadDescription settings that need to come from our QueueDescription
			NewDownloadDescription.ProgressListener = QueueDescription.ProgressListener;
			
			DownloadDescription ActiveDescription = TempRequestedDownloadsCopy.get(NewDownloadDescription.DestinationLocation);
			boolean LastRequest = (NumNewDownloadDescriptions-1) == DescriptionIndex;
			
			if (ActiveDescription == null)
			{
				//Don't even have an entry in the old list, so it's completely new
				QueueNewDownloadDescription(NewDownloadDescription, LastRequest);
				continue;
			}
			
			//Need to update our DownloadDescription
			HandleChangedDownloadDescription(ActiveDescription, NewDownloadDescription, LastRequest);
		}
	}
	
	public synchronized void ResetDownloadCompleteTracking(DownloadDescription Description)
	{
		CompletedDownloads.remove(Description.DestinationLocation);
		FailedDownloads.remove(Description.DestinationLocation);
	}

	public synchronized ECompleteReason GetCompleteReasonForDownload(DownloadDescription Description)
	{
		if (Description.bIsCancelled)
		{
			return ECompleteReason.Cancelled;
		}

		if (FailedDownloads.containsKey(Description.DestinationLocation))
		{
			return ECompleteReason.Error;
		}

		//Default to success since the download is complete and we have no signs of an error
		return ECompleteReason.Success;
	}

	public synchronized void QueueNewDownloadDescription(DownloadDescription Description, boolean LastRequest)
	{
		//We have hit this code after something else has already invalidated our FetchInstance
		if (!IsFetchInstanceValid())
		{
			Log.error("Call to QueueNewDownloadDescription after FetchInstance is invalidated! RequestID:" + Description.RequestID);
			return;
		}
		RequestedDownloads.put(Description.DestinationLocation, Description);
		ResetDownloadCompleteTracking(Description);

		if (!CheckForPreviouslyCompletedDownload(Description))
		{	
			FetchEnqueueResultListener.FetchEnqueueRequestCallback RequestCallback = new FetchEnqueueResultListener.FetchEnqueueRequestCallback(this, Description.DestinationLocation);
			FetchEnqueueResultListener.FetchEnqueueErrorCallback ErrorCallback = new FetchEnqueueResultListener.FetchEnqueueErrorCallback(this, Description.DestinationLocation);
		
			Request FetchRequest = BuildFetchRequest(Description);
			if (FetchRequest == null)
			{
				Log.error("Invalid FetchRequest generated for " + Description.RequestID +" . Ending request with error as this shouldn't happen.");
				CompleteDownload(Description, ECompleteReason.Error, true);
				return;
			}

			Description.CachedFetchID = FetchRequest.getId();

			//Needed for callbacks
			DownloadContentLength downContent = new DownloadContentLength();
			downContent.desc = Description;
			DownloadContentLengthError downContentError = new DownloadContentLengthError();
			FetchInstance.getContentLengthForRequest(FetchRequest, true, downContent, downContentError);
			
			FetchInstance.enqueue(FetchRequest, RequestCallback, ErrorCallback);

			Log.debug("Enqueue request sent for:" + Description.RequestID);
		}
		//if we have previously completed this download, then we just want to compelte it instead of creating a new FetchRequest
		else
		{
			ECompleteReason CompleteReason = GetCompleteReasonForDownload(Description);
			Log.debug("Completed RequestID:" + Description.RequestID + " with Reason " + CompleteReason + " as request was previously completed.");
			CompleteDownload(Description, CompleteReason, LastRequest);
		}
	}

	private synchronized boolean CheckForPreviouslyCompletedDownload(DownloadDescription Description)
	{
		boolean bInCompleteList = CompletedDownloads.containsKey(Description.DestinationLocation);
		
		File CompleteLocation = new File(Description.DestinationLocation);
		boolean bHasFileInCompleteLocation = CompleteLocation.exists();

		return (bInCompleteList || bHasFileInCompleteLocation || Description.bHasCompleted || Description.bIsCancelled);
	}

	@Override
	public synchronized void PauseDownload(String DestinationLocation, boolean bPause)
	{
		runningState = ETaskRunningState.Unknown;

		//Early out if we have invalidated our FetchInstance and thus would crash if trying to do work
		if (!IsFetchInstanceValid())
		{
			Log.error("Call to PauseDownload after FetchInstance has been invalidated. Skipping as no work can be done. DestinationLocation: " + DestinationLocation);
			return;
		}	

		DownloadDescription MatchedDesc = RequestedDownloads.get(DestinationLocation);
		if (null == MatchedDesc)
		{
			Log.error("No DownloadDescription found for DestinationLocation " + DestinationLocation +" during PauseDownload!");
			return;
		}
		
		MatchedDesc.bIsPaused = bPause;
		if (bPause)
		{
			FetchInstance.pause(MatchedDesc.CachedFetchID);
		}
		else
		{
			FetchInstance.resume(MatchedDesc.CachedFetchID);
		}
	}

	@Override
	public synchronized void PauseAllDownloads()
	{
		if (runningState == ETaskRunningState.Paused)
		{
			Log.debug("Fetch - PauseAllDownloads - Already paused!");
			return;
		}

		if (!IsFetchInstanceValid())
		{
			Log.debug("Fetch - PauseAllDownloads - Not valid!");
			return;
		}

		int pauseCount = 0;
		int totalCount = 0;

		for(Map.Entry<String, DownloadDescription> entry : RequestedDownloads.entrySet()) 
		{
			
    		DownloadDescription MatchedDesc = entry.getValue();
			if (null == MatchedDesc)
			{
				return;
			}
			
			if (!MatchedDesc.bIsPaused)
			{
				MatchedDesc.bIsPaused = true;
				FetchInstance.pause(MatchedDesc.CachedFetchID);

				pauseCount++;
			}

			totalCount++;
		}

		runningState = ETaskRunningState.Paused;
		Log.debug("Fetch - PauseAllDownloads - Paused " + pauseCount + " of " + totalCount);
	}

	@Override
	public synchronized void ResumeAllDownloads()
	{
		if (runningState == ETaskRunningState.Running)
		{
			Log.debug("Fetch - ResumeAllDownloads - Already resumed!");
			return;
		}
		
		if (!IsFetchInstanceValid())
		{
			Log.debug("Fetch - ResumeAllDownloads - Not valid!");
			return;
		}

		int resumeCount = 0;
		int totalCount = 0;
		
		for(Map.Entry<String, DownloadDescription> entry : RequestedDownloads.entrySet()) 
		{
			
    		DownloadDescription MatchedDesc = entry.getValue();
			if (null == MatchedDesc)
			{
				return;
			}
			
			if (MatchedDesc.bIsPaused)
			{
				MatchedDesc.bIsPaused = false;
				FetchInstance.resume(MatchedDesc.CachedFetchID);

				resumeCount++;
			}

			totalCount++;
		}

		runningState = ETaskRunningState.Running;
		Log.debug("Fetch - ResumeAllDownloads - Resumed " + resumeCount + " of " + totalCount);
	}

	@Override
	public synchronized void CancelDownload(String DestinationLocation)
	{
		if (!IsFetchInstanceValid())
		{
			Log.error("Call to CancelDownload with an invalid fetch instance. Skipping fetch work to avoid crashing. DestinationLocation: " + DestinationLocation);
			return;
		}

		DownloadDescription MatchedDesc = RequestedDownloads.get(DestinationLocation);
		if (null == MatchedDesc)
		{
			Log.error("No DownloadDescription found for DestinationLocation " + DestinationLocation +" during CancelDownload!");
			return;
		}
		
		if (false == MatchedDesc.bIsCancelled)
		{
			Log.verbose("Cancelling download:" + MatchedDesc.RequestID);
			
			//Set download as complete since it was cancelled intentionally
			MatchedDesc.PreviousDownloadPercent = 100;
			CompletedDownloads.put(DestinationLocation, MatchedDesc);
			MatchedDesc.bIsCancelled = true;

			FetchInstance.delete(MatchedDesc.CachedFetchID);
		}
		CheckForAllDownloadsComplete(MatchedDesc.ProgressListener);
	}
	
	private synchronized Request BuildFetchRequest(DownloadDescription Description)
	{
		String URL = GetNextURL(Description);
		String TempDownloadLoc = GetTempDownloadDestination(Description);

		//Can not build a valid FetchRequest with a null URL (will crash when trying to enqueue)
		if ((URL == null) || (Description.DestinationLocation == null))
		{
			Log.error("URL or DestinationLocation to build Uri from are null! Can not create a valid FetchRequest! RequestID:" + Description.RequestID);
			return null;
		}

		//Want to download the file to the DestinationLocation with the TempFileExtension appended. This gets removed when the file is finished
		Uri DownloadUri = Uri.parse(TempDownloadLoc);

		Request FetchRequest = new Request(URL, DownloadUri);
		FetchRequest.setPriority(GetFetchPriority(Description));
		FetchRequest.setTag(Description.DestinationLocation);
		FetchRequest.setGroupId(Description.GroupID);
		FetchRequest.setAutoRetryMaxAttempts(Description.IndividualURLRetryCount);
		
		//For now we don't specify this on the fetch request because its already specified on the WorkManager worker, which should stop all our
		//downloading when it stops work anyway.
		FetchRequest.setNetworkType(NetworkType.ALL);
			
		return FetchRequest;
	}
	
	private synchronized void HandleChangedDownloadDescription(DownloadDescription OldDescription, DownloadDescription NewDescription, boolean LastRequest)
	{
		if (!IsFetchInstanceValid())
		{
			Log.error("Call to HandleChangedDownloadDescription with an invalid fetch instance. Skipping to avoid crashing.");
			return;
		}

		Log.verbose("Updating download description as it has changed:" + OldDescription.RequestID);
		CopyStateToNewDescription(OldDescription, NewDescription);

		//Handle the change by cancelling and recreating the fetch2 download
		RecreateDownloadByTagFunc RecreateFunc = new RecreateDownloadByTagFunc(this, NewDescription, LastRequest);
		FetchInstance.getDownloadsByTag(OldDescription.DestinationLocation, RecreateFunc);
	}
	
	//Copies over the non-serialized values that track download state from our old descrition to our new description so that changing
	//serialized values doesn't wipe out existing downlod behavior
	private void CopyStateToNewDescription(DownloadDescription OldDescription, DownloadDescription NewDescription)
	{
		NewDescription.CurrentRetryCount = OldDescription.CurrentRetryCount;
		NewDescription.CachedFetchID = OldDescription.CachedFetchID;
		NewDescription.PreviousDownloadedBytes = OldDescription.PreviousDownloadedBytes;
		NewDescription.PreviousDownloadPercent = OldDescription.PreviousDownloadPercent;
		NewDescription.TotalBytesNeeded = OldDescription.TotalBytesNeeded;
		NewDescription.TotalDownloadedBytes = OldDescription.TotalDownloadedBytes;

		// Purposefully don't copy bIsCancelled as this isn't tracked on the C++ side and if we are asking for this to be re-queued then
		// we no longer want this download cancelled and should redo it

		//Purposefully don't copy DownloadProgressListener as we set this based on the new QueueDescription already and likely
		//don't want to keep the old one (although in current code they match).
		//NewDescription.DownloadProgressListener ProgressListener = OldDescription.DownloadProgressListener;
	}
	
	private boolean IsFetchInstanceValid()
	{
		return ((null != FetchInstance)	&& (false == FetchInstance.isClosed()));
	}
	
	private String GetNextURL(DownloadDescription Description)
	{
		if (IsOutOfRetries(Description))
		{
			Log.error("Download calling GetNextURL even though it is out of retries! Shouldn't happen!");
			return null;
		}
		
		final int NumURLs = Description.URLs.length;
		final int URLIndexToUse = Description.CurrentRetryCount % NumURLs;
		
		return Description.URLs[URLIndexToUse];
	}
	
	private boolean IsOutOfRetries(DownloadDescription Description)
	{
		return ((Description.CurrentRetryCount >= Description.MaxRetryCount) && Description.MaxRetryCount >= 0);
	}
	
	private Priority GetFetchPriority(DownloadDescription Description)
	{
		switch (Description.RequestPriority)
		{
		case 0:
			return Priority.NORMAL;
		case 1:
			return Priority.HIGH; 
		case -1:
			return Priority.LOW;
		default:
			return Priority.NORMAL;
		}
	}
	
	private synchronized void InitFetch(Context context, int MaxConcurrentDownloads)
	{
		Log.debug("InitFetch called");

		//Remove the old FetchListener if it was setup immediately so we don't
		//get any invalid callbacks while inside of the synchronized re-create of the FetchInstance
		if (FetchListener != null)
		{
			if (IsFetchInstanceValid())
			{
				FetchInstance.removeListener(FetchListener);
			}
			FetchListener = null;
		}

		if (AllowHTTP2 && okHttpClient == null)
		{
			SharedPreferences preferences = context.getSharedPreferences("BackgroundDownload", context.MODE_PRIVATE);
			AllowHTTP2 = preferences.getBoolean("AllowHTTP2", false);

			if (AllowHTTP2)
			{
				okHttpClient = new OkHttpClient.Builder().build();
			}
		}

		//Synchronized with StopWork to make sure we aren't creating and closing our fetch instance in a weird race condition
//		synchronized(this)
		{
			//Make sure any existing FetchInstance is in a correct state (null and ready to be created, with any in progress work deleted)
			if (FetchInstance != null)
			{
				if (!FetchInstance.isClosed())
				{
					DeleteAllInProgressRequests();
					FetchInstance.close();
				}
				else
				{
					Log.debug("InitFetch has existing non-closed FetchInstance!");
				}

				FetchInstance = null;
			}

			if (FetchInstance != null)
			{
				Log.error("Unexpected already configured FetchInstance after destroying FetchInstance in InitFetch!");
			}
			else
			{
				//TODO TRoss: Pull these values from the worker's getInputData
				if (!AllowHTTP2)
				{
					FetchInstance = Fetch.Impl.getInstance(new FetchConfiguration.Builder(context)
						.setNamespace(context.getPackageName())
						.enableLogging(true)
						.setLogger(FetchLog)
						.enableRetryOnNetworkGain(true)
						.setProgressReportingInterval(200)
						.setDownloadConcurrentLimit(MaxConcurrentDownloads)
						.build());
				}
				else
				{
					FetchInstance = Fetch.Impl.getInstance(new FetchConfiguration.Builder(context)
						.setNamespace(context.getPackageName())
						.enableLogging(true)
						.setLogger(FetchLog)
						.enableRetryOnNetworkGain(true)
						.setProgressReportingInterval(200)
						.setDownloadConcurrentLimit(MaxConcurrentDownloads)
						.setHttpDownloader(new OkHttpDownloader(okHttpClient))
						.build());
				}
			}
			
			if (!IsFetchInstanceValid())
			{
				Log.error("Unexpected invalid FetchInstance after completing InitFetch!");
			}
			else
			{	
				//sanity check as this should have been nulled before the synchronized block
				if (FetchListener != null)
				{
					Log.error("Unexpected FetchListener still active during InitFetch creation!");
					FetchInstance.removeListener(FetchListener);
					FetchListener = null;
				}

				FetchListener = new FetchRequestProgressListener(this);
				FetchInstance.addListener(FetchListener);
			}
		}
	}
	
	//Helper class to avoid use of Delegates as our current compile source target is 7 and thus delegates are not supported.
	//Cancels and then recreates downloads found by a given tag.
	private class RecreateDownloadByTagFunc extends CancelDownloadByTagFunc
	{
		@Override
		public void call(List<Download> MatchingDownloads)
		{
			if (IsValid()) 
			{
				//First cancel the download
				super.call(MatchingDownloads);
				
				//Now just have the fetch manager owner recreate the download with the supplied data
				Owner.QueueNewDownloadDescription(RecreateDescription, LastRequest);
			}
			else
			{
				Log.debug("Aborting RecreateDownloadByTagFunc as FetchInstance has been invalidated");
			}
		}
		
		public RecreateDownloadByTagFunc(FetchManagerOld Owner, DownloadDescription RecreateDescription, boolean LastRequest)
		{
			this.Owner = Owner;
			this.RecreateDescription = RecreateDescription;
			this.LastRequest = LastRequest;
		}
		
		private boolean IsValid()
		{
			return ((null != Owner) && (null != RecreateDescription) && IsFetchInstanceValid());
		}
		
		private boolean LastRequest;
		private FetchManagerOld Owner;
		private	DownloadDescription RecreateDescription;
	}

	private class DownloadContentLength implements Func<Long>
	{
		@Override
		public void call(@NonNull Long result) {
			desc.TotalBytesNeeded = result;
		}

		public DownloadDescription desc;
	}

	// Needed for getContentLengthForRequest.
	private class DownloadContentLengthError implements Func<Error>
	{
		@Override
		public void call(@NonNull Error result) {
		}
	}

	//Helper class to avoid use of Delegates as our current compile source target is 7 and thus delegates are not supported.
	//Cancels downloads found by a given tag.
	private class CancelDownloadByTagFunc implements Func<List<Download>>
	{
		@Override
		public void call(List<Download> MatchingDownloads)
		{
			if (MatchingDownloads != null)
			{
				for (int DownloadIndex = 0; DownloadIndex < MatchingDownloads.size(); ++DownloadIndex)
				{
					if (!IsFetchInstanceValid())
					{
						Log.debug("Aborting CancelDownloadByTagFunc as FetchInstance has been invalidated.");
						break;
					}
				
					Download FoundDownload = MatchingDownloads.get(DownloadIndex);
					FetchInstance.cancel(FoundDownload.getId());
				}
			}
		}
	}
	
	//Function that allows us to catch any errors where the FetchAPI has stopped all work and thus should bubble up that all downloads are complete
	//Employs a non-synchronous callback, so check will not be instant
	public synchronized void RequestCheckDownloadsStillActive(DownloadProgressListener ProgressListener)
	{
		if (!IsFetchInstanceValid())
		{
			Log.error("RequestCheckDownloadsStillActive called while FetchInstance is invalid! Skipping request.");
			return;
		}
		
		CheckForActiveDownloadsFunc HasActiveDownloadFunc = new CheckForActiveDownloadsFunc(this, ProgressListener);
		FetchInstance.hasActiveDownloads(true /*includeAddedDownloads*/, HasActiveDownloadFunc);
	}

	@Override
	public synchronized void RequestGroupProgressUpdate(int GroupID, DownloadProgressListener ListenerToUpdate)
	{
		long TotalBytesProgress = 0;
		long TotalBytesNeeded = 0;
		
		ArrayList<String> DownloadKeys = new ArrayList<String>(RequestedDownloads.keySet());
		for (int DescIndex = 0; DescIndex < DownloadKeys.size(); ++DescIndex)
		{
			DownloadDescription FoundDesc = RequestedDownloads.get(DownloadKeys.get(DescIndex));
			
			if (FoundDesc.GroupID == GroupID)
			{
				TotalBytesProgress += FoundDesc.TotalDownloadedBytes;
				TotalBytesNeeded += FoundDesc.TotalBytesNeeded;
			}
		}
				
		float Progress = (float)TotalBytesProgress / (float)TotalBytesNeeded;
		//Based on BPS BackgroundDownload wight is 3
		Progress = Progress * 0.75f;
		boolean bIsIndeterminate = (TotalBytesProgress == 0);
		
		//Make sure we cap at 100% progress
		if (Progress > 1.0f)
		{
			Progress = 1.0f;
		}
		ListenerToUpdate.OnDownloadGroupProgress(GroupID, (int)Math.ceil(Progress * 100.0f), bIsIndeterminate);
	}

	//
	// FetchDownloadProgressOwner Implementation
	//
	@Override
	public void OnDownloadQueued(@NonNull Download download)
	{
		Log.verbose("OnDownloadQueued: " + GetDestinationLocation(download));

		//Treat this as just an un-pause for now to make sure we handle cases where Fetch resumes downloads that we paused (This can happen when retrying for networkgain as an example)
		OnDownloadChangePauseState(download, false);
	}

	@Override
	public void OnDownloadStarted(@NonNull Download download)
	{
		String DestinationLocation = GetDestinationLocation(download);
		Log.verbose("OnDownloadStarted: " + DestinationLocation);

		DownloadDescription MatchedDownload = RequestedDownloads.get(DestinationLocation);
		if (null == MatchedDownload)
		{
			Log.error("OnDownloadStarted called from DownloadProgressOwner implementation with a download that doesn't match any DownloadDesc! Download's Tag: " + DestinationLocation);
			return;
		}

		if (MatchedDownload.DownloadStartTime == 0)
		{
			MatchedDownload.DownloadStartTime = System.currentTimeMillis();
		}
	}

	@Override
	public void OnDownloadProgress(@NonNull Download download, boolean indeterminate, long downloadedBytesPerSecond, long etaInMilliSeconds)
	{
		String DestinationLocation = GetDestinationLocation(download);

		Log.verbose("OnDownloadProgress: " + DestinationLocation + " bytes/s:" + downloadedBytesPerSecond + " eta:" + etaInMilliSeconds);

		DownloadDescription MatchedDownload = RequestedDownloads.get(DestinationLocation);
		if (null == MatchedDownload)
		{
			Log.error("OnDownloadProgress called from DownloadProgressOwner implementation with a download that doesn't match any DownloadDesc! Download's Tag: " + DestinationLocation);
			return;
		}
		
		if (HasValidProgressCallback(MatchedDownload))
		{
			long TotalDownloadedSinceLastCall = 0;
			
			long TotalDownloaded = download.getDownloaded();
			//If our TotalDownload size has gone down, we likely had an error and restarted the download since the previous update.
			//Don't want to return a negative value, so just use 0 for now
			if (MatchedDownload.TotalDownloadedBytes > TotalDownloaded)
			{
				TotalDownloadedSinceLastCall = 0;
			}
			else
			{
				TotalDownloadedSinceLastCall = (TotalDownloaded - MatchedDownload.TotalDownloadedBytes);
			}
			MatchedDownload.PreviousDownloadPercent = download.getProgress();
			MatchedDownload.PreviousDownloadedBytes = TotalDownloadedSinceLastCall;
			MatchedDownload.TotalDownloadedBytes = TotalDownloaded;
			MatchedDownload.ProgressListener.OnDownloadProgress(GetDestinationLocation(download), TotalDownloaded, TotalDownloadedSinceLastCall);
			MatchedDownload.TotalBytesNeeded = download.getTotal();
			
		}
		else
		{
			Log.error("DownloadDescription tied to download does not have a valid ProgressListener callback! RequestID:" + MatchedDownload.RequestID);
		}
	}

	@Override
	public synchronized void OnDownloadChangePauseState(@NonNull Download download, boolean bIsPaused)
	{
		runningState = ETaskRunningState.Unknown;

		String DestinationLocation = GetDestinationLocation(download);
		Log.verbose("OnDownloadChangePauseState: " + DestinationLocation);

		DownloadDescription MatchedDownload = RequestedDownloads.get(DestinationLocation);
		if (null == MatchedDownload)
		{
			Log.error("OnDownloadChangePauseState called from DownloadProgressOwner implementation with a download that doesn't match any DownloadDesc! Download's Tag: " + DestinationLocation);
			return;
		}
		
		if (!IsFetchInstanceValid())
		{
			Log.error("Call to OnDownloadChangePauseState with an invalid fetch instance. Skipping fetch work to avoid crashing. RequestID: " + MatchedDownload.RequestID);
			return;
		}

		//the DownloadDescription is always definitive, so make sure the new download state matches our expectations
		if (bIsPaused != MatchedDownload.bIsPaused)
		{
			Log.verbose("OnDownloadChangePauseState required Fetch action. RequestID:" + MatchedDownload.RequestID + " New bIsPaused:" + MatchedDownload.bIsPaused + " Old Fetch bIsPaused:" + bIsPaused);			
			
			if (MatchedDownload.bIsPaused)
			{
				MatchedDownload.DownloadPauseTime = System.currentTimeMillis();

				FetchInstance.pause(download.getId());
			}
			else
			{
				if (MatchedDownload.DownloadPauseTime != 0)
				{
					long ElapsedPauseTime = System.currentTimeMillis() - MatchedDownload.DownloadPauseTime;

					MatchedDownload.TotalPausedTime += ElapsedPauseTime;

					MatchedDownload.DownloadPauseTime = 0;
				}

				FetchInstance.resume(download.getId());
			}
		}
	}

	@Override
	public void OnDownloadGroupProgress(@NonNull FetchGroup Group, DownloadProgressListener ProgressListener)
	{
		if (null == ProgressListener)
		{
			Log.error("Call to OnDownloadGroupProgress with an invalid DownloadProgressListener!");
			return;
		}
		
		int Progress = Group.getGroupDownloadProgress();
		boolean bIsIndeterminate = (Progress > 0);
		
		ProgressListener.OnDownloadGroupProgress(Group.getId(), Progress, bIsIndeterminate);
	}
	
	//This version is the DownloadProgressListener version of this function! See CompleteDownload for FetchManagerOld implementation
	@Override
	public void OnDownloadComplete(@NonNull Download download, ECompleteReason completeReason)
	{
		String DestinationLocation = GetDestinationLocation(download);
		Log.debug("OnDownloadComplete: " + DestinationLocation + " Reason:" + completeReason);

		DownloadDescription MatchedDownload = RequestedDownloads.get(DestinationLocation);
		if (null == MatchedDownload)
		{
			Log.error("OnDownloadComplete called from DownloadProgressOwner implementation with a download that doesn't match any DownloadDesc! Download's Tag: " + DestinationLocation);
			return;
		}
		
		CompleteDownload(MatchedDownload, completeReason, true);
	}

	//
	// FetchEnqueueResultListener Implementation
	//
	@Override
	public void OnFetchEnqueueRequestCallback(@NonNull String DestinationLocation, @NonNull Request EnqueuedRequest)
	{
		Log.verbose("Enqueued Request Success. DestinationLocation:" + DestinationLocation);
		
		DownloadDescription MatchingDescription = RequestedDownloads.get(DestinationLocation);
		if (null == MatchingDescription)
		{
			Log.error("OnFetchEnqueueRequestCallback called on request is missing from our RequestedDownloads list! DestinationLocation:" + DestinationLocation);
			return;
		}

		MatchingDescription.ProgressListener.OnDownloadEnqueued(MatchingDescription);
	}

	@Override
	public synchronized void OnFetchEnqueueErrorCallback(@NonNull String DestinationLocation, @NonNull Error EnqueueError)
	{
		Log.error("Error Enqueing Request! " + DestinationLocation + " Error: " + EnqueueError);

		Throwable thrown = EnqueueError.getThrowable();
		if (thrown != null) {
			String message = thrown.getMessage();
			if (message != null && message.contains("database is locked")) {
				Log.error("Enqueue Error due to locked SQLITE database, likely from previous deleteAll request");
			}
		}	
		
		//If no real error supplied, lets check for fetch having already enqueued this in a separate listener
		if (IsFetchInstanceValid() && IsErrorPossiblyIndicatingPreviouslyQueued(EnqueueError))
		{
			Log.error("Enqueue Error " + EnqueueError + " could be because of a previous enqueue already succeeding. Checking if Request is already enqueued before erroring. " + DestinationLocation);
						
			CheckForQueuedDownload CheckForQueueFunc = new CheckForQueuedDownload(this, DestinationLocation);
			FetchInstance.getDownloadsByTag(DestinationLocation, CheckForQueueFunc);
		
			return;
		}

		DownloadDescription MatchingDescription = RequestedDownloads.get(DestinationLocation);
		if (null == MatchingDescription)
		{
			Log.error("OnFetchEnqueueErrorCallback called on request is missing from our RequestedDownloads list! DestinationLocation:" + DestinationLocation);
			return;
		}

		RetryDownload(DestinationLocation);

		MatchingDescription.ProgressListener.OnDownloadEnqueued(MatchingDescription);
	}
	
	public synchronized void HandleEnqueueCheckCallback(String DestinationLocation, @Nullable Download FetchDownload)
	{
		boolean bIsAlreadyEnqueued = false;

		//No existing download, so it wasn't enqueued previously
		if (FetchDownload == null) 
		{
			bIsAlreadyEnqueued = false;
		}
		//It was already queued (or further) and we can just complete with an enqueue success and move on
		else if ((FetchDownload.getStatus() == Status.valueOf("QUEUED"))
				|| (FetchDownload.getStatus() == Status.valueOf("DOWNLOADING"))
				|| (FetchDownload.getStatus() == Status.valueOf("COMPLETED")))
		{
			bIsAlreadyEnqueued = true;
		}
		//download exists but it's in some other state that shouldn't count as previously enqeued (such as failed, cancelled, etc)
		else
		{
			bIsAlreadyEnqueued = false;
		}

		
		DownloadDescription MatchingDescription = RequestedDownloads.get(DestinationLocation);
		
		//If we weren't already enqueued or this request does't map to an existing download we should remove it from the fetch instance
		//to try and allow any future retries to complete successfully
		if (IsFetchInstanceValid() 
			&& (FetchDownload != null) 
			&& (!bIsAlreadyEnqueued || (MatchingDescription == null)))
		{
			FetchInstance.delete(FetchDownload.getId());
			FetchInstance.remove(FetchDownload.getId());
		}
				
		if (null == MatchingDescription)
		{
			Log.debug("No DownloadDescription matches " + DestinationLocation + " can not send enqueue callback with result " + bIsAlreadyEnqueued);
			return;
		}
		
		//Update our cached fetch ID with the found one
		if (FetchDownload != null)
		{
			MatchingDescription.CachedFetchID = FetchDownload.getId();
		}

		if (!bIsAlreadyEnqueued)
		{
			RetryDownload(DestinationLocation);
		}

		MatchingDescription.ProgressListener.OnDownloadEnqueued(MatchingDescription);
	}

	public boolean IsErrorPossiblyIndicatingPreviouslyQueued(@NonNull Error EnqueueError)
	{
		//Unfortunately I have seen the UNKNOWN error returned for cases where the request is already successfully Enqueued
		//even though we have EnqueueAction set to UPDATE_ACCORDINGLY. I'm checking all these cases to cover our bases since
		//they all seem like they could be returned if a previous enqueue already enqueued the request
		Error ErrorsThatCouldBePreviousEnqueueErrors[] = {
			Error.UNKNOWN,
			Error.REQUEST_ALREADY_EXIST, 
			Error.REQUEST_WITH_ID_ALREADY_EXIST, 
			Error.REQUEST_WITH_FILE_PATH_ALREADY_EXIST, 
			Error.FAILED_TO_UPDATE_REQUEST
		};
		
		for (int ErrorIndex = 0; ErrorIndex < ErrorsThatCouldBePreviousEnqueueErrors.length; ++ErrorIndex)
		{
			if (EnqueueError == ErrorsThatCouldBePreviousEnqueueErrors[ErrorIndex])
			{
				return true;
			}
		}

		return false;
	}

	//if UEDownloadableWorker is calling into it, the only thing it should need is the DestinationLocation
	@Override
	public void RetryDownload(String DestinationLocation)
	{
		RetryDownload(DestinationLocation, null);
	}
	
	private synchronized void RetryDownload(String DestinationLocation, @Nullable Download FetchDownload)
	{
		DownloadDescription MatchingDescription = RequestedDownloads.get(DestinationLocation);
		if (null == MatchingDescription)
		{
			Log.error("RetryDownload called on invalid download that was never requested! DestinationLocation:" + DestinationLocation);
			return;
		}

		if (!IsFetchInstanceValid())
		{
			Log.debug("Call to RetryDownload with an invalid fetch instance. Skipping fetch work and completing to avoid crashing. RequestID:" + MatchingDescription.RequestID);
			CompleteDownload(MatchingDescription, FetchRequestProgressListener.ECompleteReason.Error, true);
			return;
		}

		
		MatchingDescription.CurrentRetryCount++;

		if (IsOutOfRetries(MatchingDescription))
		{
			CompleteDownload(MatchingDescription, FetchRequestProgressListener.ECompleteReason.OutOfRetries, true);
			return;
		}
		
		//if we already know the associated FetchDownload we can just pass it through
		if (null != FetchDownload)
		{
			RetryDownload_Internal(MatchingDescription, FetchDownload);
		}
		//We don't know the associated Fetch download, so lets query Fetch for it first to see if it exists
		else
		{
			//Setup a callback to RetryDownload_Internal once we try and get our Download from Fetch
			FetchInstance.getDownload(MatchingDescription.CachedFetchID, new RetryDownloadFunc(MatchingDescription));
		}
	}
	
	private synchronized void RetryDownload_Internal(@NonNull DownloadDescription DownloadDesc, @Nullable Download RetryDownload)
	{
		if (!IsFetchInstanceValid())
		{
			Log.debug("Call to RetryDownload_Internal with an invalid fetch instance. Skipping fetch work and completing to avoid crashing. RequestID:" + DownloadDesc.RequestID);
			CompleteDownload(DownloadDesc, FetchRequestProgressListener.ECompleteReason.Error, true);
			return;
		}

		if (null != RetryDownload)
		{
			//Remove existing download from Fetch
			FetchInstance.remove(RetryDownload.getId());
			
			//Remove partial download file when switching URLs and wait to recreate until the callback is finished for the delete
			FetchInstance.delete(RetryDownload.getId(),new RecreateCallbackWithDownload(DownloadDesc), new RecreateCallbackWithError(DownloadDesc));
		}
		else
		{
			QueueNewDownloadDescription(DownloadDesc, true);
		}
	}

	private class CheckForQueuedDownload implements Func<List<Download>>
	{
		public CheckForQueuedDownload(FetchManagerOld Owner, String DestinationLocation)
		{
			this.Owner = Owner;
			this.DestinationLocation = DestinationLocation;
		}

		@Override
		public void call(List<Download> MatchingDownloads)
		{
			//Callback for first found download
			if ((MatchingDownloads != null) && (MatchingDownloads.size() > 0))
			{
				Owner.HandleEnqueueCheckCallback(DestinationLocation, MatchingDownloads.get(0));
			}
			//remove and delete any other downloads that might have been found after the first
			synchronized (this.Owner)
			{
				for (int DownloadIndex = 1; DownloadIndex < MatchingDownloads.size(); ++DownloadIndex)
				{
					if (!IsFetchInstanceValid())
					{
						return;
					}
					
					Download FoundDownload = MatchingDownloads.get(DownloadIndex);
					FetchInstance.remove(FoundDownload.getId());
					FetchInstance.delete(FoundDownload.getId());
				}
			}
		}

		private FetchManagerOld Owner;
		private String DestinationLocation = null;
	}

	private class RetryDownloadFunc implements Func2<Download>
	{
		public RetryDownloadFunc (DownloadDescription CachedDownloadDescription)
		{
			this.CachedDownloadDescription = CachedDownloadDescription;
		}

		@Override
		public void call(@Nullable Download MatchingDownload)
		{
			RetryDownload_Internal(CachedDownloadDescription, MatchingDownload);
		}

		private DownloadDescription CachedDownloadDescription = null;
	}

	
	private class RecreateCallbackBase
	{
		public RecreateCallbackBase(DownloadDescription CachedDownloadDescription)
		{
			this.CachedDownloadDescription = CachedDownloadDescription;
		}
		private void call_internal()
		{
			if (null != CachedDownloadDescription) 
			{
				QueueNewDownloadDescription(CachedDownloadDescription, true);
			}
		}
		
		private DownloadDescription CachedDownloadDescription = null;
	}
	private class RecreateCallbackWithDownload extends RecreateCallbackBase implements Func<Download>
	{
		public RecreateCallbackWithDownload(DownloadDescription CachedDownloadDescription)
		{
			super(CachedDownloadDescription);
		}
		
		@Override
		public void call(@NonNull Download CancelledDownload)
		{
			super.call_internal();
		}
	}
	private class RecreateCallbackWithError extends RecreateCallbackBase implements Func<Error>
	{
		public RecreateCallbackWithError(DownloadDescription CachedDownloadDescription)
		{
			super(CachedDownloadDescription);
		}
		
		@Override
		public void call(@Nullable Error CancelError)
		{
			super.call_internal();
		}
	}

	//Helper class to handle routing calls from the FetchInstance's hasActiveDownloads function back to our FetchManagerOld
	private class CheckForActiveDownloadsFunc implements Func<Boolean>
	{
		@Override
		public void call(Boolean bHasActiveDownloads)
		{
			if (null != bHasActiveDownloads)
			{
				if (IsValid())
				{
					Owner.HandleHasActiveDownloadResult(bHasActiveDownloads, ProgressListener);
				}
			}
		}

		public CheckForActiveDownloadsFunc(FetchManagerOld Owner, DownloadProgressListener ProgressListener)
		{
			this.Owner = Owner;
			this.ProgressListener = ProgressListener;
		}

		private boolean IsValid()
		{
			return ((null != Owner) && (null != ProgressListener));
		}

		private FetchManagerOld Owner;
		private DownloadProgressListener ProgressListener;
	}
	
	private String GetTempDownloadDestination(DownloadDescription DownloadDesc)
	{
		return DownloadDesc.DestinationLocation + TempFileExtension;
	}

	private void HandleHasActiveDownloadResult(boolean bHasActiveDownloads, DownloadProgressListener ProgressListener)
	{
		if (!bHasActiveDownloads)
		{
			Log.debug("No active downloads in FetchInstance, but still waiting on downloads to finish!");
		}
	}

	private synchronized void CompleteDownload(DownloadDescription DownloadDesc, ECompleteReason CompleteReason, boolean CheckForAllCompleted)
	{
		if (false == HasValidProgressCallback(DownloadDesc))
		{
			Log.error("Call to CompleteDownload with an invalid DownloadDescription! RequestID:" + DownloadDesc.RequestID);
			return;
		}
		
		EDownloadCompleteReason CompleteReasonToSend = ConvertCompleteReasonForDownload(CompleteReason);
		
		//Mark download as complete
		if (CompleteReason == ECompleteReason.Success)
		{
			DownloadDesc.PreviousDownloadPercent = 100;
		}
		CompletedDownloads.put(DownloadDesc.DestinationLocation, DownloadDesc);

		//Only bubble up to the UEDownloadWorker non-intenional completes as we initiated any other completes internally
		if (IsCompleteReasonIntentional(CompleteReason))
		{
			Log.verbose("Call to CompleteDownload taking no action as complete reason was intentional! RequestID:" + DownloadDesc.RequestID + " CompleteReason:" + CompleteReason);
			return;
		}

		//Since this download was completed, lets reset it's CurrentRetryCount, this way if it's requeued
		//by any future work before this DownloadDescription is removed, it will correctly retry the download
		//with the correct amount of retries
		//NOTE: This is happening after the IsCompleteReasonIntentional check because we don't want worker requeues or other intentional download removal/delete/cancels
		//to clear out the CurrentRetryCount as we didn't actually finish this download yet and need to keep the retry state for when it is restarted.
		DownloadDesc.CurrentRetryCount = 0;

		//Test if we were successful, and if so lets try and move our successful download to the non-temp location
		if (CompleteReason == FetchRequestProgressListener.ECompleteReason.Success)
		{
			String TempDownloadLoc = GetTempDownloadDestination(DownloadDesc);
			File CompletedFile = new File(TempDownloadLoc);
			File NewFileAtDestinationLocation = new File(DownloadDesc.DestinationLocation);

			if (NewFileAtDestinationLocation.exists())
			{
				Log.debug("Completed download that already exists! Completing with original file " + DownloadDesc.DestinationLocation);
				
				if (CompletedFile.exists())
				{
					Log.error("Completed download that already existed was also re-downloaded! Should not happen! Deleting dupicate tempfile at " + TempDownloadLoc);
					CompletedFile.delete();
				}
			}
			else
			{
				//Attempt rename, and if it fails we need to treat this download as an error
				if (!CompletedFile.renameTo(NewFileAtDestinationLocation))
				{
					Log.error("Failed to rename " + TempDownloadLoc + " to " + DownloadDesc.DestinationLocation + " ! Download has Errored.");
				
					//Overwrite our successful complete reason with an error.
					CompleteReasonToSend = EDownloadCompleteReason.Error;
					FailedDownloads.put(DownloadDesc.DestinationLocation, DownloadDesc);
				}
			}
		}
		else
		{
			FailedDownloads.put(DownloadDesc.DestinationLocation, DownloadDesc);
		}

		long DownloadDuration = -1;
		long DownloadFinishTime = System.currentTimeMillis();

		if (DownloadDesc.DownloadStartTime > 0)
		{
			long ElapsedDownloadTime = DownloadFinishTime - DownloadDesc.DownloadStartTime;

			DownloadDuration = ElapsedDownloadTime - DownloadDesc.TotalPausedTime;
		}

		DownloadDesc.ProgressListener.OnDownloadMetrics(DownloadDesc.DestinationLocation, DownloadDesc.TotalDownloadedBytes, DownloadDuration, DownloadDesc.DownloadStartTime, DownloadFinishTime);

		DownloadDesc.ProgressListener.OnDownloadComplete(DownloadDesc.DestinationLocation, CompleteReasonToSend);
		
		if (CheckForAllCompleted)
		{
			CheckForAllDownloadsComplete(DownloadDesc.ProgressListener);
		}
	}
	
	private EDownloadCompleteReason ConvertCompleteReasonForDownload(ECompleteReason CompleteReason)
	{
		switch (CompleteReason)
		{
			case Error:
				return EDownloadCompleteReason.Error; 
			case Success:
				return EDownloadCompleteReason.Success;
			case OutOfRetries:
				return EDownloadCompleteReason.OutOfRetries;
			default:
				return EDownloadCompleteReason.Error;
		}
	}
	
	private synchronized void CheckForAllDownloadsComplete(DownloadProgressListener ProgressListener)
	{
		if (CompletedDownloads.size() < RequestedDownloads.size())
		{
			return;
		}

		if (CompletedDownloads.size() > RequestedDownloads.size())
		{
			Log.error("Error in CompleteDownload logic! CompletedDownloads.size:" + CompletedDownloads.size() + " RequstedDownloads.size():" + RequestedDownloads.size());
		}
		
		SendAllDownloadsCompleteNotification(ProgressListener, false);
	}

	private void SendAllDownloadsCompleteNotification(DownloadProgressListener ProgressListener, boolean bForceFailure)
	{
		boolean bDidAllSucceed = ((bForceFailure == false) && (FailedDownloads.isEmpty()));
		ProgressListener.OnAllDownloadsComplete(bDidAllSucceed);
		
		if (AllDownloadsCompletedListener != null)
		{
			AllDownloadsCompletedListener.OnAllDownloadsCompleted(bDidAllSucceed);
		}
	}
	
	private String GetDestinationLocation(Download download)
	{
		if (null != download)
		{
			return download.getTag();
		}
		
		return "";
	}

	private boolean HasValidProgressCallback(DownloadDescription DownloadDesc)
	{
		return ((null != DownloadDesc) && (null != DownloadDesc.ProgressListener));
	}

	//Used to determine if the reason a download completed was because of action we took. IE: Cancelling, deleting, removing, etc that happened through calls to the FetchManagerOld.
	//Or if it was a change of status by hitting an error, succeeding, etc.
	private boolean IsCompleteReasonIntentional(FetchRequestProgressListener.ECompleteReason CompleteReason)
	{
		switch (CompleteReason)
		{
			//All cases that are intentional should be here
			case Cancelled:
			case Deleted:
			case Removed:
				return true;
			//by default assume its not intentional
			default:
				return false;
		}
	}
	
	@Override
	public synchronized void SetAllDownloadsCompletedListener(@NonNull AllDownloadsCompletedListener Listener)
	{
		AllDownloadsCompletedListener = Listener;
	}
	
	//Wrapper class for our FetchInstance logging that passes the values through UE's Logger Class
	//FetchLogger only has debug and error logging currently implemented so that is all we pass through.
	private class FUEFetchLogger extends FetchLogger
	{
		public FUEFetchLogger()
		{
			//Disable logging because we only want this overridden logging behavior
			//and not the base logging that happens with FetchLogger
			super(false, "UEFetchLoggerWrapper");
		}

		private Logger UELogger = new Logger("UE", "FetchAPI");

		@Override 
		public void d(String message)
		{
			UELogger.debug(message);
		}

		@Override
		public void d(String message, Throwable throwable)
		{
			//Since we have a throwable, send this as an error with throwable as our UE Logger class doesn't have a
			//debug with throwable
			UELogger.error(message, throwable);
		}

		@Override
		public void e(String message)
		{
			UELogger.error(message);
		}

		@Override
		public void e(String message, Throwable throwable)
		{
			UELogger.error(message, throwable);
		}
	}
	
	private static volatile FetchManagerOld SharedManager = new FetchManagerOld();
	
	private volatile Fetch FetchInstance = null;

	private volatile FetchRequestProgressListener FetchListener = null;
	private volatile AllDownloadsCompletedListener AllDownloadsCompletedListener = null;

	private volatile OkHttpClient okHttpClient = null;
	private volatile boolean AllowHTTP2 = true;

	private volatile ETaskRunningState runningState = ETaskRunningState.Unknown;

	private volatile HashMap<String, DownloadDescription> RequestedDownloads = new HashMap<String, DownloadDescription>();
	private volatile HashMap<String, DownloadDescription> CompletedDownloads = new HashMap<String, DownloadDescription>();
	private volatile HashMap<String, DownloadDescription> FailedDownloads = new HashMap<String, DownloadDescription>();

	public String TempFileExtension = ".fetchtemp";
	
	public Logger Log = new Logger("UE", "FetchManagerOld");
	public FUEFetchLogger FetchLog = new FUEFetchLogger();
}