// Copyright Epic Games, Inc. All Rights Reserved.
package com.epicgames.unreal.download.fetch;

import android.util.Log;

import androidx.annotation.NonNull;
import androidx.annotation.Nullable;

import com.tonyodev.fetch2.Download;
import com.tonyodev.fetch2.Error;
import com.tonyodev.fetch2.FetchListener;
import com.tonyodev.fetch2.exception.FetchException;
import com.tonyodev.fetch2core.DownloadBlock;

import org.jetbrains.annotations.NotNull;

import java.util.List;

import com.epicgames.unreal.Logger;
import com.epicgames.unreal.download.fetch.FetchDownloadProgressOwner;

//progress listener passed into Fetch2 to update the progress of our downloads. Really just simple wrapper that forwards back to the DownloadProgressOwner.
//Could just implement FetchListener on our UEDownloadWorker, but not sure that we want to add WorkManager Worker objects as Fetch2 listeners directly.
final class FetchRequestProgressListener implements FetchListener
{
	public enum ECompleteReason
	{
		Success,
		Error,
		OutOfRetries,
		Cancelled,
		Removed,
		Deleted;
	}
	
	private static Logger Log = new Logger("UE", "FetchRequestProgressListener");
	
	public void BroadcastDownloadQueued(@NonNull Download download)
	{
		Owner.OnDownloadQueued(download);
	}

	public void BroadcastDownloadStarted(@NonNull Download download)
	{
		Owner.OnDownloadStarted(download);
	}
	
	public void BroadcastProgress(@NonNull Download download, boolean indeterminate)
	{
		Owner.OnDownloadProgress(download, indeterminate, CachedDownloadedBytesPerSecond, CachedETAInMilliSeconds);
	}
	
	public void BroadcastComplete(@NonNull Download download, ECompleteReason completeReason)
	{
		Owner.OnDownloadComplete(download, completeReason);
	}
	
	public void BroadcastChangePauseState(@NonNull Download download, boolean bIsPaused)
	{
		Owner.OnDownloadChangePauseState(download, bIsPaused);
	}
	
	
	@Override
	public void onQueued(@NonNull Download download, boolean waitingOnNetwork)
	{
		Log.verbose("onQueued");
		BroadcastDownloadQueued(download);
	}

	@Override
	public void onCompleted(@NonNull Download download)
	{
		Log.verbose("onCompleted");
		BroadcastComplete(download, ECompleteReason.Success);
	}

	@Override
	public void onError(@NonNull Download download, Error error, Throwable throwable)
	{
		if (throwable != null)
		{
			Log.verbose("OnError (exception)");
		}
		else
		{
			Log.verbose("onError (no exception)");
		}

		BroadcastComplete(download, ECompleteReason.Error);
	}

	@Override
	public void onStarted(@NonNull Download download, @NonNull List<? extends DownloadBlock> list, int i)
	{
		//Make sure to broadcast a change in pause state, because if we previously thought we were paused we aren't anymore!
		Log.verbose("onStarted");
		BroadcastDownloadStarted(download);
		BroadcastChangePauseState(download, false);
	}
	
	@Override
	public void onProgress(@NonNull Download download, long etaInMilliSeconds, long downloadedBytesPerSecond)
	{
		Log.verbose("onProgress eta:" + etaInMilliSeconds + " Bytes/s:" + downloadedBytesPerSecond);
		
		//Update our cache
		CachedDownloadedBytesPerSecond = downloadedBytesPerSecond;
		CachedETAInMilliSeconds = etaInMilliSeconds;
	
		BroadcastProgress(download, false);
	}

	@Override
	public void onPaused(@NonNull Download download)
	{
		Log.verbose("onPaused");
		BroadcastChangePauseState(download, true);
	}

	@Override
	public void onResumed(@NonNull Download download)
	{
		Log.verbose("onResumed...");
		
		if (download.getStatus() == com.tonyodev.fetch2.Status.FAILED)
		{
			Log.verbose("onResumed had an error, broadcasting complete with error!");
			BroadcastComplete(download, ECompleteReason.Error);
		}
		else
		{
			Log.verbose("onResumed broadcasting normal resume");
			BroadcastChangePauseState(download, false);
		}
	}

	@Override
	public void onCancelled(Download download)
	{
		Log.verbose("onCancelled");
		BroadcastComplete(download, ECompleteReason.Cancelled);
	}

	@Override
	public void onRemoved(@NonNull Download download)
	{
		Log.verbose("onRemoved");
		BroadcastComplete(download, ECompleteReason.Removed);
	}

	@Override
	public void onDeleted(@NonNull Download download)
	{
		Log.verbose("onDeleted");
		BroadcastComplete(download, ECompleteReason.Deleted);
	}

	@Override
	public void onWaitingNetwork(@NonNull Download download)
	{
		//Treat waiting on a network as being paused by the system
		Log.verbose("onWaitingNetwork");
		BroadcastChangePauseState(download, true);
	}
	
	//
	//Currently unused overrides
	//
	@Override
	public void onAdded(@NonNull Download download)
	{
		//For now don't need to do anything in OnAdded, just wait for OnQueued
	}

	@Override
	public void onDownloadBlockUpdated(@NonNull Download download, @NonNull DownloadBlock downloadBlock, int i)
	{
		//Not currently using this for anything useful
	}
	
	FetchRequestProgressListener(@NonNull FetchDownloadProgressOwner Owner)
	{
		this.Owner = Owner;
	}

	private FetchDownloadProgressOwner Owner = null;
	
	long CachedDownloadedBytesPerSecond = 0;
	long CachedETAInMilliSeconds = 0;
}