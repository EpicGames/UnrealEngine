// Copyright Epic Games, Inc. All Rights Reserved.

#include "AndroidDownloadManager.h"

#include "Misc/ConfigCacheIni.h"
#include "Misc/CoreDelegates.h"

#include "BackgroundHttpModule.h"
#include "PlatformBackgroundHttp.h"

#include "Android/AndroidPlatform.h"
#include "Android/AndroidJavaEnv.h"
#include "Android/AndroidPlatformFile.h"

#include "Containers/UnrealString.h"
#include "Misc/Paths.h"

#include "BuildSettings.h"
#include "Async/Async.h"
#include "HAL/FileManager.h"
#include "HAL/PlatformFileManager.h"

namespace UE::Jni::Download::DataStructs
{
	struct FDownloadDescription: Java::Lang::FObject
	{
		static constexpr FAnsiStringView ClassName = "com/epicgames/unreal/download/datastructs/DownloadDescription";
		
		inline static TConstructor<FDownloadDescription(Java::Lang::FString* RequestID, Java::Lang::FString* DestinationLocation, jint MaxRetryCount, jint IndividualURLRetryCount, jint RequestPriority, jint GroupID, Java::Lang::TArray<Java::Lang::FString*>* URLs, jboolean bIsPaused, jlong TotalBytesNeeded)> New;
		
		static constexpr FMember Members[]
		{
			UE_JNI_MEMBER(New)
		};
	};
}

namespace UE::Jni::Download
{
	struct FDownloadManager: Java::Lang::FObject
	{
		static constexpr FAnsiStringView ClassName = "com/epicgames/unreal/download/DownloadManager";

		inline static TMember<FDownloadManager, void (Java::Util::FHashMap* Params, jboolean bWaitTillDone)> Initialize;
		inline static TMember<FDownloadManager, void (jint MaxConcurrentDownloads)> SetMaxConcurrentDownloads;
		inline static TMember<FDownloadManager, void (DataStructs::FDownloadDescription* DownloadDescription)> Enqueue;
		inline static TMember<FDownloadManager, void (Java::Lang::FString* DestinationLocation)> Pause;
		inline static TMember<FDownloadManager, void (Java::Lang::FString* DestinationLocation)> Resume;
		inline static TMember<FDownloadManager, void (Java::Lang::FString* DestinationLocation)> Cancel;
		inline static TMember<FDownloadManager, Java::Lang::FString* ()> GetActiveDestinations;

		static void JNICALL OnStart(JNIEnv* env, Java::Lang::TClass<FDownloadManager>* clazz, Java::Lang::FString* DestinationLocation, Java::Lang::FString* DebugString, jlong StartTimeUTC)
		{
			AsyncTask(ENamedThreads::GameThread, [DestinationLocation = Env.Marshal(DestinationLocation), DebugString = Env.Marshal(DebugString), StartTimeUTC]
			{
				if (TSharedPtr<Online::Download::Android::FDownloadManager> Manager = StaticCastSharedPtr<Online::Download::Android::FDownloadManager>(FBackgroundHttpModule::Get().GetBackgroundHttpManager()))
				{
					Manager->OnStart(DestinationLocation, DebugString, StartTimeUTC);
				}
			});
		}

		static void JNICALL OnProgress(JNIEnv* env, Java::Lang::TClass<FDownloadManager>* clazz, Java::Lang::FString* DestinationLocation, jlong TotalBytesWritten, jlong BytesWrittenSinceLastCall)
		{
			AsyncTask(ENamedThreads::GameThread, [DestinationLocation = Env.Marshal(DestinationLocation), TotalBytesWritten, BytesWrittenSinceLastCall]
			{
				if (TSharedPtr<Online::Download::Android::FDownloadManager> Manager = StaticCastSharedPtr<Online::Download::Android::FDownloadManager>(FBackgroundHttpModule::Get().GetBackgroundHttpManager()))
				{
					Manager->OnProgress(DestinationLocation, TotalBytesWritten, BytesWrittenSinceLastCall);
				}
			});
		}
		
		static void JNICALL OnComplete(JNIEnv* env, Java::Lang::TClass<FDownloadManager>* clazz, Java::Lang::FString* DestinationLocation, jboolean bDidSucceed, jlong TotalBytesDownloaded, jlong DownloadDuration, jlong DownloadStartTimeUTC, jlong DownloadEndTimeUTC)
		{
			AsyncTask(ENamedThreads::GameThread, [DestinationLocation = Env.Marshal(DestinationLocation), bDidSucceed, TotalBytesDownloaded, DownloadDuration, DownloadStartTimeUTC, DownloadEndTimeUTC]
			{
				if (TSharedPtr<Online::Download::Android::FDownloadManager> Manager = StaticCastSharedPtr<Online::Download::Android::FDownloadManager>(FBackgroundHttpModule::Get().GetBackgroundHttpManager()))
				{
					Manager->OnComplete(DestinationLocation, bDidSucceed, TotalBytesDownloaded, DownloadDuration, DownloadStartTimeUTC, DownloadEndTimeUTC);
				}
			});
		}

		static constexpr FMember Members[]
		{
			UE_JNI_MEMBER(Initialize),
			UE_JNI_MEMBER(SetMaxConcurrentDownloads),
			UE_JNI_MEMBER(Enqueue),
			UE_JNI_MEMBER(Pause),
			UE_JNI_MEMBER(Resume),
			UE_JNI_MEMBER(Cancel),
			UE_JNI_MEMBER(GetActiveDestinations)
		};

		static constexpr FNativeMethod NativeMethods[]
		{
			UE_JNI_NATIVE_METHOD(OnStart),
			UE_JNI_NATIVE_METHOD(OnProgress),
			UE_JNI_NATIVE_METHOD(OnComplete)
		};
	};
}

#if USE_ANDROID_JNI
extern void AndroidThunkCpp_SetSharedPreferenceString(const FString& Group, const FString& Key, const FString& Value);
#endif

namespace UE::Online::Download::Android
{
	bool FDownloadManager::FRequest::ProcessRequest()
	{
		if (!ensureAlwaysMsgf(IsInGameThread(), TEXT("Should only ever call ProcessRequest from GameThread! Cannot Process!")))
		{
			return false;
		}
		
		StaticCastSharedPtr<FDownloadManager>(FBackgroundHttpModule::Get().GetBackgroundHttpManager())->EnqueueRequest(*this);
		return true;
	}
	
	void FDownloadManager::FRequest::CancelRequest()
	{
		if (!ensureAlwaysMsgf(IsInGameThread(), TEXT("Should only ever call CancelRequest from GameThread! Cannot Cancel!")))
		{
			return;
		}
		
		if (TSharedPtr<FDownloadManager> Manager = Parent.Pin())
		{
			Manager->CancelRequest(*this);
		}
	}

	void FDownloadManager::FRequest::PauseRequest()
	{
		if (!ensureAlwaysMsgf(IsInGameThread(), TEXT("Should only ever call PauseRequest from GameThread! Cannot Pause!")))
		{
			return;
		}
		
		UE_LOGF(LogBackgroundHttpManager, Verbose, "PauseRequest called on %ls", *GetRequestID());

		bIsPaused = true;
		
		if (Parent.IsValid())
		{
			Jni::Download::FDownloadManager::Pause(Jni::Class<Jni::Download::FDownloadManager>, *Jni::Env.Marshal(DestinationLocation));
		}
	}

	void FDownloadManager::FRequest::ResumeRequest()
	{
		if (!ensureAlwaysMsgf(IsInGameThread(), TEXT("Should only ever call ResumeRequest from GameThread! Cannot Resume!")))
		{
			return;
		}
		
		UE_LOGF(LogBackgroundHttpManager, Verbose, "ResumeRequest called on %ls", *GetRequestID());

		bIsPaused = false;

		if (Parent.IsValid())
		{
			Jni::Download::FDownloadManager::Resume(Jni::Class<Jni::Download::FDownloadManager>, *Jni::Env.Marshal(DestinationLocation));
		}
	}

#if !UE_BUILD_SHIPPING
	void FDownloadManager::FRequest::GetDebugText(TArray<FString>& Output)
	{
		const FString CurrentURL = URLList.Num() > 0 ? URLList[0] : TEXT("no url");

		const double DownloadedSizeMB = (double)DownloadedBytes / (1024.0 * 1024.0);
		const FString DownloadedString = FString::Printf(TEXT(" %.2fMB"), DownloadedSizeMB);

		const double ExpectedSizeMB = (double)GetExpectedResultSize() / (1024.0 * 1024.0);
		const FString ExpectedString = (ExpectedSizeMB > 0.0) ? FString::Printf(TEXT("/%.2fMB"), ExpectedSizeMB) : TEXT("");

		FString DetailSpeed;
		if (DownloadStartTime > 0.0 && DownloadedBytes > 0)
		{
			const double Elapsed = FPlatformTime::Seconds() - DownloadStartTime;
			if (Elapsed > 0.0)
			{
				const double MBps = DownloadedSizeMB / Elapsed;
				DetailSpeed = FString::Printf(TEXT(" %.2fMB/s"), MBps);
			}
		}

		Output.Add(FString::Printf(TEXT("%s\n%s p%d%s%s%s %s"),
			*CurrentURL,
			bIsPaused ? TEXT("P") : TEXT("R"),
			GetPriorityAsAndroidPriority(),
			*DownloadedString,
			*ExpectedString,
			*DetailSpeed,
			*DebugString));
	}
#endif

	int FDownloadManager::FRequest::GetPriorityAsAndroidPriority() const
	{
		switch (RequestPriority)
		{
		case EBackgroundHTTPPriority::High:
			return 1;
		case EBackgroundHTTPPriority::Low:
			return -1;
		case EBackgroundHTTPPriority::Normal:
			return 0;
		default:
			ensureAlwaysMsgf(false, TEXT("Missing EBackgroundHTTPPriority in GetPriorityAsAndroidPriority!"));
			return 0;
		}
	}

	FDownloadManager::FDownloadManager()
		: MaxActiveDownloads{4}
		, FileHashHelper{MakeShared<FBackgroundHttpFileHashHelper, ESPMode::ThreadSafe>()}
	{
		Jni::Env.Initialize<Jni::Download::DataStructs::FDownloadDescription>();
		Jni::Env.Initialize<Jni::Download::FDownloadManager>();
	}

	void FDownloadManager::Initialize(bool bClearPreExistingRequestsAtStartup)
	{
		AndroidThunkCpp_SetSharedPreferenceString(TEXT("BackgroundPreferences"), TEXT("TempDownloadRootPath"), FBackgroundHttpFileHashHelper().GetTemporaryRootPath());

		ensureAlwaysMsgf(GConfig->GetInt(TEXT("BackgroundHttp"), TEXT("MaxActiveDownloads"), MaxActiveDownloads, GEngineIni), TEXT("No value found for MaxActiveDownloads! Defaulting to 4!"));
		
		AndroidBackgroundHTTPManagerDefaultLocalizedText.InitFromIniSettings("AndroidFetchBackgroundDownload");
		
		{
			using namespace Jni;
			
			FScopedJavaObject<Java::Util::FHashMap*> Params = Java::Util::FHashMap::New(Class<Java::Util::FHashMap>);

			//Set our MaxActiveDownloads in the underlying java layer to match our expectation
			Java::Util::FHashMap::put(*Params, *Env.Marshal(FAndroidNativeDownloadWorkerParameterKeys::DOWNLOAD_MAX_CONCURRENT_REQUESTS_KEY), *Java::Lang::FInteger::New(Class<Java::Lang::FInteger>, MaxActiveDownloads));

			//Make sure we pass in localized notification text bits for the important worker keys
			Java::Util::FHashMap::put(*Params, *Env.Marshal(FAndroidNativeDownloadWorkerParameterKeys::NOTIFICATION_CONTENT_TITLE_KEY), *Env.Marshal(AndroidBackgroundHTTPManagerDefaultLocalizedText.DefaultNotificationText_Title.GetText().ToString()));
			Java::Util::FHashMap::put(*Params, *Env.Marshal(FAndroidNativeDownloadWorkerParameterKeys::NOTIFICATION_CONTENT_COMPLETE_TEXT_KEY), *Env.Marshal(AndroidBackgroundHTTPManagerDefaultLocalizedText.DefaultNotificationText_Complete.GetText().ToString()));
			Java::Util::FHashMap::put(*Params, *Env.Marshal(FAndroidNativeDownloadWorkerParameterKeys::NOTIFICATION_CONTENT_CANCEL_DOWNLOAD_TEXT_KEY), *Env.Marshal(AndroidBackgroundHTTPManagerDefaultLocalizedText.DefaultNotificationText_Cancel.GetText().ToString()));
			Java::Util::FHashMap::put(*Params, *Env.Marshal(FAndroidNativeDownloadWorkerParameterKeys::NOTIFICATION_CONTENT_NO_INTERNET_TEXT_KEY), *Env.Marshal(AndroidBackgroundHTTPManagerDefaultLocalizedText.DefaultNotificationText_NoInternet.GetText().ToString()));
			Java::Util::FHashMap::put(*Params, *Env.Marshal(FAndroidNativeDownloadWorkerParameterKeys::NOTIFICATION_CONTENT_AIRPLANE_MODE_TEXT_KEY), *Env.Marshal(AndroidBackgroundHTTPManagerDefaultLocalizedText.DefaultNotificationText_AirplaneMode.GetText().ToString()));
			Java::Util::FHashMap::put(*Params, *Env.Marshal(FAndroidNativeDownloadWorkerParameterKeys::NOTIFICATION_CONTENT_DATA_SAVER_ENABLED_TEXT_KEY), *Env.Marshal(AndroidBackgroundHTTPManagerDefaultLocalizedText.DefaultNotificationText_DataSaverEnabled.GetText().ToString()));
			Java::Util::FHashMap::put(*Params, *Env.Marshal(FAndroidNativeDownloadWorkerParameterKeys::NOTIFICATION_CONTENT_WAITING_FOR_CELLULAR_TEXT_KEY), *Env.Marshal(AndroidBackgroundHTTPManagerDefaultLocalizedText.DefaultNotificationText_WaitingForCellular.GetText().ToString()));
			Java::Util::FHashMap::put(*Params, *Env.Marshal(FAndroidNativeDownloadWorkerParameterKeys::NOTIFICATION_CONTENT_APPROVE_TEXT_KEY), *Env.Marshal(AndroidBackgroundHTTPManagerDefaultLocalizedText.DefaultNotificationText_Approve.GetText().ToString()));

			//Expect our ContentText to have a {DownloadPercent} argument in it by default, so this will replace that with the Java string format argument so Java can insert the appropriate value
			FFormatNamedArguments Arguments;
			Arguments.Emplace(TEXT("DownloadPercent"), FText::FromString(TEXT("%3d%%")));
			FText UpdatedContentText = FText::Format(AndroidBackgroundHTTPManagerDefaultLocalizedText.DefaultNotificationText_Content.GetText(), Arguments);
			Java::Util::FHashMap::put(*Params, *Env.Marshal(FAndroidNativeDownloadWorkerParameterKeys::NOTIFICATION_CONTENT_TEXT_KEY), *Env.Marshal(UpdatedContentText.ToString()));

			Jni::Download::FDownloadManager::Initialize(Class<Jni::Download::FDownloadManager>, *Params, bClearPreExistingRequestsAtStartup);
		}
		
		FileHashHelper->LoadData();
		
		if (bClearPreExistingRequestsAtStartup)
		{
			TArray<FString> FilesToDelete;

			//Default implementation is to just delete everything in the Root Folder non-recursively.
			IPlatformFile& PlatformFile = FPlatformFileManager::Get().GetPlatformFile();
			PlatformFile.FindFiles(FilesToDelete, *FBackgroundHttpFileHashHelper::GetTemporaryDownloadPath(), nullptr);

			for (const FString& File : FilesToDelete)
			{
				UE_LOGF(LogBackgroundHttpManager, Log, "Deleting File:%ls", *File);
				const bool bDidDelete = PlatformFile.DeleteFile(*File);

				if (!bDidDelete)
				{
					UE_LOGF(LogBackgroundHttpManager, Warning, "Failure to Delete Temp File:%ls", *File);
				}
			}
		}		
		
		DeleteStaleTempFiles();

		// we download to our temporary root path, and then in a subdirectory named after our branch name;
		// this way, the fetch managers can clean up stuff they don't know (e.g. downloaded temp files from older versions)
		if (IAndroidPlatformFile::GetPlatformPhysical().CreateDirectory(*FBackgroundHttpFileHashHelper::GetTemporaryRootPath()))
		{
			const FString BuildDownloadPath = FPaths::Combine(FBackgroundHttpFileHashHelper::GetTemporaryRootPath(), BuildSettings::GetBranchName());
			if (!IAndroidPlatformFile::GetPlatformPhysical().CreateDirectory(*BuildDownloadPath))
			{
				UE_LOGF(LogBackgroundHttpManager, Display, "Failed to create build temp download path: %ls", *BuildDownloadPath);
			}
		}
	}

	void FDownloadManager::Shutdown()
	{
		ensureAlwaysMsgf(IsInGameThread(), TEXT("Called from un-expected thread! Potential error in an implementation of background downloads!"));
		
		Requests.Empty();
	}

	void FDownloadManager::AddRequest(const FBackgroundHttpRequestPtr Request)
	{
		ensureAlwaysMsgf(IsInGameThread(), TEXT("Called from un-expected thread! Potential error in an implementation of background downloads!"));
		
		EnqueueRequest(static_cast<FRequest&>(*Request));
	}

	void FDownloadManager::RemoveRequest(const FBackgroundHttpRequestPtr Request)
	{
		ensureAlwaysMsgf(IsInGameThread(), TEXT("Called from un-expected thread! Potential error in an implementation of background downloads!"));
		
		CancelRequest(static_cast<FRequest&>(*Request));		
	}

	void FDownloadManager::DeleteAllTemporaryFiles()
	{
		UE_LOGF(LogBackgroundHttpManager, Log, "Cleaning Up Temporary Files");

		TArray<FString> FilesToDelete;

		//Default implementation is to just delete everything in the Root Folder non-recursively.
		IPlatformFile& PlatformFile = FPlatformFileManager::Get().GetPlatformFile();
		PlatformFile.FindFiles(FilesToDelete, *FBackgroundHttpFileHashHelper::GetTemporaryDownloadPath(), *FBackgroundHttpFileHashHelper::GetTempFileExtension());

		for (const FString& File : FilesToDelete)
		{
			UE_LOGF(LogBackgroundHttpManager, Log, "Deleting File:%ls", *File);
			const bool bDidDelete = PlatformFile.DeleteFile(*File);

			if (!bDidDelete)
			{
				UE_LOGF(LogBackgroundHttpManager, Warning, "Failure to Delete Temp File:%ls", *File);
			}
		}
	}

	void FDownloadManager::SetCellularPreference(int32 Value)
	{
		FPlatformMisc::SetCellularPreference(Value);
	}

	void FDownloadManager::SetLimitedDataPreference(int32 Value)
	{
	}

	void FDownloadManager::SetMaxActiveDownloads(int32 Value)
	{
		check(Value > 0);
		if (MaxActiveDownloads != Value)
		{
			MaxActiveDownloads = Value;
			
			Jni::Download::FDownloadManager::SetMaxConcurrentDownloads(Jni::Class<Jni::Download::FDownloadManager>, MaxActiveDownloads);
		}
	}

	void FDownloadManager::CleanUpDataAfterCompletingRequest(const FBackgroundHttpRequestPtr Request)
	{
		//Need to free up all these URL's hashes in FileHashHelper so that future URLs can use those temp files
		for (const FString& URL : Request->GetURLList())
		{
			FileHashHelper->RemoveURLMapping(URL, Request->GetRequestID());
		}
		
		FileHashHelper->SaveData();
	}
	
	void FDownloadManager::FRequest::RotateURLList()
	{
		if (URLList.Num() > 1)
		{
			FString First = MoveTemp(URLList[0]);
			URLList.RemoveAt(0, EAllowShrinking::No);
			URLList.Add(MoveTemp(First));
		}
	}

	void FDownloadManager::EnqueueRequest(FRequest& _Request)
	{
		TSharedPtr<FRequest> Request = SharedThis(&_Request);
		
		UE_LOGF(LogBackgroundHttpRequest, Verbose, "Processing Request - RequestID:%ls", *Request->RequestID);
		
		if (!ensure(!Request->URLList.IsEmpty()))
		{
			Request->CompleteWithExistingResponseData(FPlatformBackgroundHttp::ConstructBackgroundResponse(EHttpResponseCodes::Unknown, {}));
			return;
		}
		
		for (const FString& URL : Request->GetURLList())
		{
			if (FileHashHelper->FindTempFilenameMappingForURL(URL, Request->GetRequestID(), Request->DestinationLocation))
			{
				Request->DestinationLocation = FBackgroundHttpFileHashHelper::GetFullPathOfTempFilename(Request->DestinationLocation);
				
				if (IPlatformFile& PlatformFile = FPlatformFileManager::Get().GetPlatformFile(); PlatformFile.FileExists(*Request->DestinationLocation))
				{
					int64 ExistingFileSize = PlatformFile.FileSize(*Request->DestinationLocation);
					UE_LOGF(LogBackgroundHttpManager, Display, "Found existing background task to associate with! RequestID:%ls | ExistingFileSize:%lld | ExistingFilePath:%ls", *Request->GetRequestID(), ExistingFileSize, *Request->DestinationLocation);

					Request->OnProgressUpdated().ExecuteIfBound(Request, ExistingFileSize, ExistingFileSize);
					Request->SetMetricsExtended(
					{
						.TotalBytesDownloaded = int32(ExistingFileSize),
						.DownloadDuration = 0.0f,
						.FetchStartTimeUTC = FDateTime::FromUnixTimestamp(0),
						.FetchEndTimeUTC = FDateTime::FromUnixTimestamp(0)
					});
					Request->CompleteWithExistingResponseData(FPlatformBackgroundHttp::ConstructBackgroundResponse(EHttpResponseCodes::Ok, Request->DestinationLocation));
					return;
				}
				
				break;
			}
		}
		
		if (Request->DestinationLocation.IsEmpty())
		{
			Request->DestinationLocation = FBackgroundHttpFileHashHelper::GetFullPathOfTempFilename(FileHashHelper->FindOrAddTempFilenameMappingForURL(Request->URLList[0], Request->RequestID));
			FileHashHelper->SaveData();
		}
		
		if (!ensure(Requests.FindOrAdd(Request->DestinationLocation, Request) == Request))
		{
			Request->CompleteWithExistingResponseData(FPlatformBackgroundHttp::ConstructBackgroundResponse(EHttpResponseCodes::Unknown, {}));
			return;
		}
		
		Request->Parent = StaticCastWeakPtr<FDownloadManager>(AsWeak());
		
		static constexpr int DefaultIndividualURLRetryCount = 3;
		
		Jni::Download::FDownloadManager::Enqueue(Jni::Class<Jni::Download::FDownloadManager>, *Jni::Download::DataStructs::FDownloadDescription::New(Jni::Class<Jni::Download::DataStructs::FDownloadDescription>, *Jni::Env.Marshal(Request->RequestID), *Jni::Env.Marshal(Request->DestinationLocation), Request->NumberOfTotalRetries, DefaultIndividualURLRetryCount, Request->GetPriorityAsAndroidPriority(), 0, *Jni::Env.Marshal(Request->GetURLList()), Request->bIsPaused, Request->GetExpectedResultSize()));
	}

	void FDownloadManager::CancelRequest(FRequest& _Request)
	{
		if (TSharedPtr<FRequest> Request; Requests.RemoveAndCopyValue(_Request.DestinationLocation, Request))
		{
			UE_LOGF(LogBackgroundHttpManager, Verbose, "CancelRequest called on %ls", *Request->RequestID);
			
			Request->Parent = nullptr;
		
			Jni::Download::FDownloadManager::Cancel(Jni::Class<Jni::Download::FDownloadManager>, *Jni::Env.Marshal(Request->DestinationLocation));
		}
	}
	
	void FDownloadManager::RequeueRequest(const FBackgroundHttpRequestPtr GenericRequest)
	{
		ensureAlwaysMsgf(IsInGameThread(), TEXT("RequeueRequest called from unexpected thread!"));

		FRequest& Request = static_cast<FRequest&>(*GenericRequest);

		UE_LOGF(LogBackgroundHttpManager, Display, "RequeueRequest - RequestID:%ls", *Request.RequestID);

		// Cancel existing Java-side download
		CancelRequest(Request);

		// Rotate URL list so the retry hits a different CDN
		Request.RotateURLList();

		// Re-enqueue with fresh Java-side download
		EnqueueRequest(Request);
	}

#if !UE_BUILD_SHIPPING
	void FDownloadManager::RestartAllDownloads()
	{
		ensureAlwaysMsgf(IsInGameThread(), TEXT("RestartAllDownloads called from unexpected thread!"));

		UE_LOGF(LogBackgroundHttpManager, Display, "RestartAllDownloads: cycling %d active request(s)", Requests.Num());

		// RequeueRequest mutates Requests via CancelRequest, so snapshot first.
		TArray<TSharedPtr<FRequest>> Snapshot;
		Snapshot.Reserve(Requests.Num());
		for (const TPair<FStringView, TSharedPtr<FRequest>>& Pair : Requests)
		{
			Snapshot.Add(Pair.Value);
		}

		for (const TSharedPtr<FRequest>& Request : Snapshot)
		{
			RequeueRequest(Request);
		}
	}
#endif

	void FDownloadManager::GetActiveDownloadRequestIDs(TSet<FString>& OutActiveIDs) const
	{
		OutActiveIDs.Reset();

#if USE_ANDROID_JNI
		using namespace UE;
		FScopedJavaObject<Jni::Java::Lang::FString*> ActiveDestinations = Jni::Download::FDownloadManager::GetActiveDestinations(Jni::Class<Jni::Download::FDownloadManager>);
		if (ActiveDestinations)
		{
			FString DestinationList = Jni::Env.Marshal(*ActiveDestinations);
			if (!DestinationList.IsEmpty())
			{
				TArray<FString> Destinations;
				DestinationList.ParseIntoArray(Destinations, TEXT("\n"));

				for (const FString& Destination : Destinations)
				{
					if (TSharedPtr<FRequest> Request = Requests.FindRef(Destination))
					{
						OutActiveIDs.Add(Request->GetRequestID());
					}
				}
			}
		}
#endif
	}

	void FDownloadManager::OnStart(const FString& DestinationLocation, const FString& InDebugString, int64 InStartTimeUTC) const
	{
		if (TSharedPtr<FRequest> Request = Requests.FindRef(DestinationLocation))
		{
#if !UE_BUILD_SHIPPING
			Request->DebugString = InDebugString;
#endif
			Request->StartTimeUTC = InStartTimeUTC;
		}
	}

	void FDownloadManager::OnProgress(const FString& DestinationLocation, int64 TotalBytesWritten, int64 BytesWrittenSinceLastCall) const
	{
		UE_LOGF(LogBackgroundHttpManager, VeryVerbose, "Download Progress... DestinationLocation:%ls BytesWrittenSinceLastCall:%lld TotalBytesWritten:%lld", *DestinationLocation, BytesWrittenSinceLastCall, TotalBytesWritten);
		
		if (TSharedPtr<FRequest> Request = Requests.FindRef(DestinationLocation))
		{
			Request->DownloadedBytes = TotalBytesWritten;

			if (Request->DownloadStartTime == 0.0)
			{
				Request->DownloadStartTime = FPlatformTime::Seconds();
			}

			Request->OnProgressUpdated().ExecuteIfBound(Request, TotalBytesWritten, BytesWrittenSinceLastCall);
		}
	}
	
	void FDownloadManager::OnComplete(const FString& DestinationLocation, bool bDidSucceed, int64 TotalBytesDownloaded, int64 DownloadDuration, int64 DownloadStartTimeUTC, int64 DownloadEndTimeUTC)
	{
		UE_LOGF(LogBackgroundHttpManager, Log, "DownloadMetrics... DestinationLocation:%ls TotalBytesDownloaded:%ld DownloadDuration:%ld", *DestinationLocation, TotalBytesDownloaded, DownloadDuration);
		UE_LOGF(LogBackgroundHttpManager, Log, "DownloadComplete... DestinationLocation:%ls bWasSuccess:%d", *DestinationLocation, int(bDidSucceed));
		
		if (TSharedPtr<FRequest> Request; Requests.RemoveAndCopyValue(DestinationLocation, Request))
		{
			Request->Parent = nullptr;
				
			EHttpResponseCodes::Type ResponseCodeToUse = bDidSucceed && FPlatformFileManager::Get().GetPlatformFile().FileExists(*Request->DestinationLocation) ? EHttpResponseCodes::Ok : EHttpResponseCodes::Unknown;
			
			Request->NotifyRequestMetricsExtendedAvailable(
			{
				.TotalBytesDownloaded = (int32)TotalBytesDownloaded,
				.DownloadDuration = (float)((double)DownloadDuration / 1000.0f),
				.FetchStartTimeUTC = FDateTime::FromUnixTimestampDecimal((double)DownloadStartTimeUTC / 1000.0f),
				.FetchEndTimeUTC = FDateTime::FromUnixTimestampDecimal((double)DownloadEndTimeUTC / 1000.0f)
			});
			Request->CompleteWithExistingResponseData(FPlatformBackgroundHttp::ConstructBackgroundResponse(ResponseCodeToUse, Request->DestinationLocation));
		}
		else
		{
			UE_LOGF(LogBackgroundHttpManager, Log, "Taking no action as DestinationLocation:%ls did not have a corresponding active Request", *DestinationLocation);
		}
	}
	
	void FDownloadManager::DeleteStaleTempFiles() const
	{
		//Parse our .ini values to determine how much we clean in this function
		double FileAgeTimeOutSettings = -1;
		bool bDeleteTimedOutFiles;
		bool bDeleteTempFilesWithoutURLMappings = false;
		bool bRemoveURLMappingEntriesWithoutPhysicalTempFiles = false;
		{
			GConfig->GetDouble(TEXT("BackgroundHttp"), TEXT("TempFileTimeOutSeconds"), FileAgeTimeOutSettings, GEngineIni);
			bDeleteTimedOutFiles = (FileAgeTimeOutSettings >= 0);
			
			GConfig->GetBool(TEXT("BackgroundHttp"), TEXT("DeleteTempFilesWithoutURLMappingEntries"), bDeleteTempFilesWithoutURLMappings, GEngineIni);
			
			GConfig->GetBool(TEXT("BackgroundHttp"), TEXT("RemoveURLMappingEntriesWithoutPhysicalTempFiles"), bRemoveURLMappingEntriesWithoutPhysicalTempFiles, GEngineIni);
			
			UE_LOGF(LogBackgroundHttpManager, Log, "Stale Settings -- TempFileTimeOutSeconds:%f DeleteTempFilesWithoutURLMappingEntries:%d RemoveURLMappingEntriesWithoutPhysicalTempFiles:%d", static_cast<float>(FileAgeTimeOutSettings),static_cast<int>(bDeleteTempFilesWithoutURLMappings),static_cast<int>(bRemoveURLMappingEntriesWithoutPhysicalTempFiles));
		}
		const bool bWillDoAnyWork = bDeleteTimedOutFiles || bDeleteTempFilesWithoutURLMappings || bRemoveURLMappingEntriesWithoutPhysicalTempFiles;
		
		//Only bother gathering temp files if we will actually be doing something with them
		TArray<FString> AllTempFilesToCheck;
		if (bWillDoAnyWork)
		{
			GatherAllTempFilenames(AllTempFilesToCheck);
			UE_LOGF(LogBackgroundHttpManager, Display, "Found %d temp download files.", AllTempFilesToCheck.Num());
		}
		
		//Handle all timed out files based on the .ini time out settings
		//can be turned off by setting
		if (bDeleteTimedOutFiles)
		{
			TArray<FString> TimedOutFiles;
			GatherTempFilesOlderThen(TimedOutFiles, FileAgeTimeOutSettings, &AllTempFilesToCheck);
		
			TArray<FString> TimeOutDeleteFullPaths;
			ConvertAllTempFilenamesToFullPaths(TimeOutDeleteFullPaths, TimedOutFiles);
			
			for (const FString& FullFilePath : TimeOutDeleteFullPaths)
			{
				if (IFileManager::Get().Delete(*FullFilePath))
				{
					UE_LOGF(LogBackgroundHttpManager, Log, "Successfully deleted %ls due to time out settings", *FullFilePath);
				}
				else
				{
					UE_LOGF(LogBackgroundHttpManager, Error, "Failed to delete timed out file %ls", *FullFilePath);
				}
			}
			
			//Should remove these files from the list of files we are checking as we know they are already invalid from timing out, so we shouldn't check them twice
			for(const FString& RemovedFile : TimedOutFiles)
			{
				AllTempFilesToCheck.Remove(RemovedFile);
			}
		}
		
		//Handle all temp files that should be removed because they are missing a corresponding URL mapping
		if (bDeleteTempFilesWithoutURLMappings)
		{
			TArray<FString> MissingURLMappingFiles;
			GatherTempFilesWithoutURLMappings(MissingURLMappingFiles, &AllTempFilesToCheck);
			
			TArray<FString> MissingURLDeleteFullPaths;
			ConvertAllTempFilenamesToFullPaths(MissingURLDeleteFullPaths, MissingURLMappingFiles);
			
			for (const FString& FullFilePath : MissingURLDeleteFullPaths)
			{
				if (IFileManager::Get().Delete(*FullFilePath))
				{
					UE_LOGF(LogBackgroundHttpManager, Log, "Successfully deleted %ls due to missing a URL mapping for this temp data", *FullFilePath);
				}
				else
				{
					UE_LOGF(LogBackgroundHttpManager, Error, "Failed to delete file %ls that was being deleted due to a missing URL mapping", *FullFilePath);
				}
			}
			
			//Should remove these files from the list of files we are checking as we know they are already invalid from timing out, so we shouldn't check them twice
			for(const FString& RemovedFile : MissingURLMappingFiles)
			{
				AllTempFilesToCheck.Remove(RemovedFile);
			}
		}
		
		if (bRemoveURLMappingEntriesWithoutPhysicalTempFiles)
		{
			//Remove all URL map entries that don't correspond to a physical file on disk
			FileHashHelper->DeleteURLMappingsWithoutTempFiles();
			FileHashHelper->SaveData();
		}
		
		UE_LOGF(LogBackgroundHttpManager, Log, "Kept %d temp download files:", AllTempFilesToCheck.Num());
		for (const FString& ValidFile : AllTempFilesToCheck)
		{
			UE_LOGF(LogBackgroundHttpManager, Verbose, "Kept: %ls", *ValidFile);
		}
	}

	void FDownloadManager::GatherTempFilesOlderThen(TArray<FString>& OutTimedOutTempFilenames,double SecondsToConsiderOld, const TArray<FString>* OptionalFileList /* = nullptr */)
	{
		OutTimedOutTempFilenames.Empty();
		
		TArray<FString> GatheredFullFilePathFiles;
		
		//OptionalFileList was not supplied so we need to gather all temp files to check as full file paths
		if (nullptr == OptionalFileList)
		{
			GatherAllTempFilenames(GatheredFullFilePathFiles, true);
		}
		//We supplied an OptionalFileList, but we still need a full file path list for this operation
		else
		{
			ConvertAllTempFilenamesToFullPaths(GatheredFullFilePathFiles, *OptionalFileList);
		}

		if (SecondsToConsiderOld >= 0)
		{
			UE_LOGF(LogBackgroundHttpManager, Verbose, "Checking for BackgroundHTTP temp files that are older then: %lf", SecondsToConsiderOld);
			
			for (const FString& FullFilePath : GatheredFullFilePathFiles)
			{
				FFileStatData FileData = IFileManager::Get().GetStatData(*FullFilePath);
				FTimespan TimeSinceCreate = FDateTime::UtcNow() - FileData.CreationTime;

				const double FileAge = TimeSinceCreate.GetTotalSeconds();
				const bool bShouldReturn = (FileAge > SecondsToConsiderOld);
				if (bShouldReturn)
				{
					UE_LOGF(LogBackgroundHttpManager, Verbose, "FoundTempFile: %ls with age %lf", *FullFilePath, FileAge);
					
					//Need to save output as just filename to be consistent with other functions
					OutTimedOutTempFilenames.Add(FPaths::GetCleanFilename(FullFilePath));
				}
			}
		}
	}

	void FDownloadManager::GatherTempFilesWithoutURLMappings(TArray<FString>& OutTempFilesMissingURLMappings, TArray<FString>* OptionalFileList /*= nullptr */) const
	{
		OutTempFilesMissingURLMappings.Empty();

		TArray<FString>* FileListToCheckPtr = OptionalFileList;

		//OptionalFileList was not supplied so we need to gather all temp files to check
		TArray<FString> GatheredFiles;
		if (nullptr == FileListToCheckPtr)
		{
			GatherAllTempFilenames(GatheredFiles, false);
			FileListToCheckPtr = &GatheredFiles;
		}
		
		TArray<FString>& FilesToCheckRef = *FileListToCheckPtr;
		for (const FString& File : FilesToCheckRef)
		{
			FString FoundURL = {};
			if (!FileHashHelper->FindMappedURLForTempFilename(File, FoundURL))
			{
				OutTempFilesMissingURLMappings.Add(File);
			}
		}
	}

	void FDownloadManager::GatherAllTempFilenames(TArray<FString>& OutAllTempFilenames, bool bOutputAsFullPaths /* = false */)
	{
		OutAllTempFilenames.Empty();
		
		const FString& DirectoryToCheck = FBackgroundHttpFileHashHelper::GetTemporaryDownloadPath();
		
		TArray<FString> AllFilenames;
		IFileManager::Get().FindFiles(AllFilenames, *DirectoryToCheck, *FBackgroundHttpFileHashHelper::GetTempFileExtension());
		
		//Make into full paths for output
		for (const FString& Filename : AllFilenames)
		{
			if (bOutputAsFullPaths)
			{
				OutAllTempFilenames.Add(FPaths::Combine(DirectoryToCheck, Filename));
			}
			else
			{
				OutAllTempFilenames.Add(Filename);
			}
		}
	}

	void FDownloadManager::ConvertAllTempFilenamesToFullPaths(TArray<FString>& OutFilenamesAsFullPaths, const TArray<FString>& FilenamesToConvertToFullPaths)
	{
		//Store this separately so we don't get bad behavior if the same Array is supplied for both parameters
		TArray<FString> FilenamesToOutput;

		for(const FString& ExistingFilename : FilenamesToConvertToFullPaths)
		{
			FilenamesToOutput.Add(FBackgroundHttpFileHashHelper::GetFullPathOfTempFilename(ExistingFilename));
		}

		OutFilenamesAsFullPaths = FilenamesToOutput;
	}
}

#if !UE_BUILD_SHIPPING
void BackgroundFetchDownload_RestartAllDownloads()
{
	using namespace UE::Online::Download::Android;
	if (TSharedPtr<FDownloadManager> Manager = StaticCastSharedPtr<FDownloadManager>(FBackgroundHttpModule::Get().GetBackgroundHttpManager()))
	{
		Manager->RestartAllDownloads();
	}
}
#endif
