// Copyright Epic Games, Inc. All Rights Reserved.

#include "AndroidFetchBackgroundDownloadModule.h"


void FAndroidFetchBackgroundDownloadModule::StartupModule()
{
	FeatureWrapper = MakeShared<FAndroidPlatformBackgroundHttpModularFeatureWrapper>();
	FeatureWrapper->RegisterAsModularFeature();
}

void FAndroidFetchBackgroundDownloadModule::ShutdownModule()
{
	if (FeatureWrapper.IsValid())
	{
		FeatureWrapper->UnregisterAsModularFeature();
	}

	FeatureWrapper.Reset();
}

IMPLEMENT_MODULE(FAndroidFetchBackgroundDownloadModule, AndroidFetchBackgroundDownload);