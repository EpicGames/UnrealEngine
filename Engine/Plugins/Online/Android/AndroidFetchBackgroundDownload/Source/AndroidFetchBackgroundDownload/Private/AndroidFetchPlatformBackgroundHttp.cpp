// Copyright Epic Games, Inc. All Rights Reserved.

#include "AndroidFetchPlatformBackgroundHttp.h"

#include "AndroidDownloadManager.h"
#include "AndroidPlatformBackgroundHttpManager.h"
#include "AndroidPlatformBackgroundHttpRequest.h"
#include "AndroidPlatformBackgroundHttpResponse.h"

#if USE_ANDROID_JNI
extern bool AndroidThunkCpp_GetSharedPreferenceBool(const FString& Group, const FString& Key, bool DefaultValue);
#endif

static bool UseNewDownloadManager()
{
	static bool Value = AndroidThunkCpp_GetSharedPreferenceBool("BackgroundPreferences", "UseNewDownloadManager", false);
	return Value;
}

void FAndroidFetchPlatformBackgroundHttp::Initialize()
{
}

void FAndroidFetchPlatformBackgroundHttp::Shutdown()
{
}

FBackgroundHttpManagerPtr FAndroidFetchPlatformBackgroundHttp::CreatePlatformBackgroundHttpManager()
{
	if (UseNewDownloadManager())
	{
		return MakeShared<UE::Online::Download::Android::FDownloadManager, ESPMode::ThreadSafe>();
	}

	return MakeShared<FAndroidPlatformBackgroundHttpManager, ESPMode::ThreadSafe>();
}

FBackgroundHttpRequestPtr FAndroidFetchPlatformBackgroundHttp::ConstructBackgroundRequest()
{
	if (UseNewDownloadManager())
	{
		return MakeShared<UE::Online::Download::Android::FDownloadManager::FRequest, ESPMode::ThreadSafe>();
	}
	
	return MakeShared<FAndroidPlatformBackgroundHttpRequest, ESPMode::ThreadSafe>();
}

FBackgroundHttpResponsePtr FAndroidFetchPlatformBackgroundHttp::ConstructBackgroundResponse(int32 ResponseCode, const FString& TempFilePath)
{
	if (UseNewDownloadManager())
	{
		return MakeShared<FBackgroundHttpResponseImpl, ESPMode::ThreadSafe>(TempFilePath, ResponseCode);
	}
	
	return MakeShared<FAndroidPlatformBackgroundHttpResponse, ESPMode::ThreadSafe>(ResponseCode, TempFilePath);
}

bool FAndroidFetchPlatformBackgroundHttp::CheckRequirementsSupported()
{
	return FAndroidPlatformBackgroundHttpManager::HandleRequirementsCheck();
}