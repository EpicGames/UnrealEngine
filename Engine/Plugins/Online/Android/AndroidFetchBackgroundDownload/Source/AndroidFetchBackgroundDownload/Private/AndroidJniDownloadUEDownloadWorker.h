// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#if USE_ANDROID_JNI
#include "Android/AndroidJavaEnv.h"

namespace UE::Jni::Download
{
	struct FUEDownloadWorker: Java::Lang::FObject
	{
		static constexpr FAnsiStringView ClassName = "com/epicgames/unreal/download/UEDownloadWorker";

		static void JNICALL nativeAndroidBackgroundDownloadOnWorkerStart(JNIEnv* env, jobject thiz, jstring WorkID);
		static void JNICALL nativeAndroidBackgroundDownloadOnWorkerStop(JNIEnv* env, jobject thiz, jstring WorkID);
		static void JNICALL nativeAndroidBackgroundDownloadOnProgress(JNIEnv* env, jobject thiz, jstring TaskID, jlong TotalBytesWritten, jlong BytesWrittenSinceLastCall);
		static void JNICALL nativeAndroidBackgroundDownloadOnComplete(JNIEnv* env, jobject thiz, jstring TaskID, jboolean bWasSuccess);
		static void JNICALL nativeAndroidBackgroundDownloadOnMetrics(JNIEnv* env, jobject thiz, jstring TaskID, jlong TotalBytesDownloaded, jlong DownloadDuration, jlong DownloadStartTimeUTC, jlong DownloadEndTimeUTC);
		static void JNICALL nativeAndroidBackgroundDownloadOnAllComplete(JNIEnv* env, jobject thiz, jboolean bDidAllRequestsSucceed);
		static void JNICALL nativeAndroidBackgroundDownloadOnTick(JNIEnv* env, jobject thiz);

		static constexpr FNativeMethod NativeMethods[]
		{
			UE_JNI_NATIVE_METHOD(nativeAndroidBackgroundDownloadOnWorkerStart),
			UE_JNI_NATIVE_METHOD(nativeAndroidBackgroundDownloadOnWorkerStop),
			UE_JNI_NATIVE_METHOD(nativeAndroidBackgroundDownloadOnProgress),
			UE_JNI_NATIVE_METHOD(nativeAndroidBackgroundDownloadOnComplete),
			UE_JNI_NATIVE_METHOD(nativeAndroidBackgroundDownloadOnMetrics),
			UE_JNI_NATIVE_METHOD(nativeAndroidBackgroundDownloadOnAllComplete),
			UE_JNI_NATIVE_METHOD(nativeAndroidBackgroundDownloadOnTick)
		};
	};
}

namespace UE::Jni
{
	template struct TInitialize<Download::FUEDownloadWorker>;
}
#endif
