// Copyright Epic Games, Inc. All Rights Reserved.

#include "AndroidPlatformBackgroundHttpManager.h"

#include "Misc/ConfigCacheIni.h"
#include "Misc/ScopeRWLock.h"
#include "Misc/CoreDelegates.h"

#include "UEWorkManagerNativeWrapper.h"
#include "PlatformBackgroundHttp.h"
#include "AndroidJniDownloadUEDownloadWorker.h"
#include "AndroidPlatformBackgroundHttpRequest.h"

#include "Interfaces/IHttpResponse.h"

#include "Android/AndroidPlatform.h"
#include "Android/AndroidJava.h"
#include "Android/AndroidJavaEnv.h"
#include "Android/AndroidJNI.h"
#include "Android/AndroidApplication.h"
#include "Android/AndroidPlatformFile.h"

#include "Containers/UnrealString.h"
#include "Misc/Paths.h"

#include "HAL/IConsoleManager.h"

#include "BuildSettings.h"

#include "HAL/PlatformProcess.h"

#if USE_ANDROID_JNI
extern void AndroidThunkCpp_SetSharedPreferenceString(const FString& Group, const FString& Key, const FString& Value);
#endif

#if UE_BUILD_SHIPPING
// always clear any exceptions in shipping
#define CHECK_JNI_RESULT(Id) if (Id == 0) { Env->ExceptionClear(); }
#else
#define CHECK_JNI_RESULT(Id)								\
	if (Id == 0)											\
	{														\
		if (!bIsOptional)									\
		{													\
			Env->ExceptionDescribe();						\
			checkf(Id != 0, TEXT("Failed to find " #Id));	\
		}													\
		Env->ExceptionClear();								\
	}
#endif // UE_BUILD_SHIPPING

#define LOCTEXT_NAMESPACE "AndroidBackgroundHttpManager"

static TAutoConsoleVariable<bool> CVarEnableCellularHandling(
	TEXT("bgdl.EnableCellularHandling"),
	false,
	TEXT("Enable/disable Cellular handling in the Java layer"),
	ECVF_Default);
static TAutoConsoleVariable<bool> CVarStopOldJobBeforeStartingNew(
	TEXT("bgdl.StopOldJobBeforeStartingNew"),
	true,
	TEXT("Enable/disable stopping for old background jobs before starting new"),
	ECVF_Default);

//Call backs called by the JNI java systems to call back into C++ code in this AndroidPlatformBackgroundHttpManager
namespace FAndroidBackgroundDownloadDelegates
{
	DECLARE_DELEGATE_TwoParams(FAndroidBackgroundDownload_OnWorkerStart, FString /*WorkID*/, jobject /*UnderlyingWorker*/);
	DECLARE_DELEGATE_TwoParams(FAndroidBackgroundDownload_OnWorkerStop, FString /*WorkID*/, jobject /*UnderlyingWorker*/);
	DECLARE_DELEGATE_FourParams(FAndroidBackgroundDownload_OnProgress, jobject /*UnderlyingWorker*/, FString /*DestinationLocation*/, int64_t /*TotalBytesWritten*/, int64_t /*BytesWrittenSinceLastCall*/);
	DECLARE_DELEGATE_ThreeParams(FAndroidBackgroundDownload_OnComplete, jobject /*UnderlyingWorker*/, FString /*DestinationLocation*/, bool /*bWasSuccess*/);
	DECLARE_DELEGATE_SixParams(FAndroidBackgroundDownload_OnMetrics, jobject /*UnderlyingWorker*/, FString /*DestinationLocation*/, int64_t /*TotalBytesDownloaded*/, int64_t /*DownloadDuration*/, int64_t /*DownloadStartTimeUTC*/, int64_t /*DownloadEndTimeUTC*/);
	DECLARE_DELEGATE_TwoParams(FAndroidBackgroundDownload_OnAllComplete, jobject /*UnderlyingWorker*/, bool /*bDidAllRequestsSucceed*/);
	DECLARE_DELEGATE_TwoParams(FAndroidBackgroundDownload_OnTickWorkerThread, JNIEnv*, jobject /*UnderlyingWorker*/);

	//Delegates called by JNI functions to bubble up underlying java work to the manager
	FAndroidBackgroundDownload_OnWorkerStart AndroidBackgroundDownload_OnWorkerStart;
	FAndroidBackgroundDownload_OnWorkerStop AndroidBackgroundDownload_OnWorkerStop;
	FAndroidBackgroundDownload_OnProgress AndroidBackgroundDownload_OnProgress;
	FAndroidBackgroundDownload_OnComplete AndroidBackgroundDownload_OnComplete;
	FAndroidBackgroundDownload_OnMetrics AndroidBackgroundDownload_OnMetrics;
	FAndroidBackgroundDownload_OnAllComplete AndroidBackgroundDownload_OnAllComplete;
	FAndroidBackgroundDownload_OnTickWorkerThread AndroidBackgroundDownload_OnTickWorkerThread;
};

const FString FAndroidPlatformBackgroundHttpManager::BackgroundHTTPWorkID = TEXT("BackgroundHttpDownload");
const FString FAndroidPlatformBackgroundHttpManager::AndroidBackgroundDownloadConfigRulesSettingKey = TEXT("AndroidBackgroundDownloadSetting");
volatile int32 FAndroidPlatformBackgroundHttpManager::bHasManagerScheduledBGWork = false;

const FString FAndroidNativeDownloadWorkerParameterKeys::DOWNLOAD_DESCRIPTION_LIST_KEY = TEXT("DownloadDescriptionList");
const FString FAndroidNativeDownloadWorkerParameterKeys::DOWNLOAD_MAX_CONCURRENT_REQUESTS_KEY = TEXT("MaxConcurrentDownloadRequests");

const FString FAndroidNativeDownloadWorkerParameterKeys::NOTIFICATION_CHANNEL_ID_KEY = TEXT("NotificationChannelId");
const FString FAndroidNativeDownloadWorkerParameterKeys::NOTIFICATION_CHANNEL_NAME_KEY = TEXT("NotificationChannelName");
const FString FAndroidNativeDownloadWorkerParameterKeys::NOTIFICATION_CHANNEL_IMPORTANCE_KEY = TEXT("NotificationChannelImportance");

const FString FAndroidNativeDownloadWorkerParameterKeys::NOTIFICATION_ID_KEY = TEXT("NotificationId");

//random value that our NOTIFICATION_ID is set to if not provided using the above key. KEEP IN SYNC WITH DownloadWorkerParameterKeys.java
const int FAndroidNativeDownloadWorkerParameterKeys::NOTIFICATION_DEFAULT_ID_KEY = 1923901283;

const FString FAndroidNativeDownloadWorkerParameterKeys::CELLULAR_HANDLE_KEY = TEXT("CellularHandleKey");

const FString FAndroidNativeDownloadWorkerParameterKeys::NOTIFICATION_CONTENT_TITLE_KEY = TEXT("NotificationContentTitle");
const FString FAndroidNativeDownloadWorkerParameterKeys::NOTIFICATION_CONTENT_TEXT_KEY = TEXT("NotificationContentText");
const FString FAndroidNativeDownloadWorkerParameterKeys::NOTIFICATION_CONTENT_CANCEL_DOWNLOAD_TEXT_KEY = TEXT("NotificationContentCancelDownloadText");
const FString FAndroidNativeDownloadWorkerParameterKeys::NOTIFICATION_CONTENT_NO_INTERNET_TEXT_KEY = TEXT("NotificationContentNoInternetDownloadText");
const FString FAndroidNativeDownloadWorkerParameterKeys::NOTIFICATION_CONTENT_AIRPLANE_MODE_TEXT_KEY = TEXT("NotificationContentAirmodeModeText");
const FString FAndroidNativeDownloadWorkerParameterKeys::NOTIFICATION_CONTENT_DATA_SAVER_ENABLED_TEXT_KEY = TEXT("NotificationContentDataSaverEnabledText");
const FString FAndroidNativeDownloadWorkerParameterKeys::NOTIFICATION_CONTENT_COMPLETE_TEXT_KEY = TEXT("NotificationContentCompleteText");
const FString FAndroidNativeDownloadWorkerParameterKeys::NOTIFICATION_CONTENT_WAITING_FOR_CELLULAR_TEXT_KEY = TEXT("NotificationContentWaitingForCellularText");
const FString FAndroidNativeDownloadWorkerParameterKeys::NOTIFICATION_CONTENT_APPROVE_TEXT_KEY = TEXT("NotificationContentApproveText");

const FString FAndroidNativeDownloadWorkerParameterKeys::NOTIFICATION_RESOURCE_CANCEL_ICON_NAME = TEXT("NotificationResourceCancelIconName");
const FString FAndroidNativeDownloadWorkerParameterKeys::NOTIFICATION_RESOURCE_CANCEL_ICON_TYPE = TEXT("NotificationResourceCancelIconType");
const FString FAndroidNativeDownloadWorkerParameterKeys::NOTIFICATION_RESOURCE_CANCEL_ICON_PACKAGE = TEXT("NotificationResourceCancelIconPackage");

const FString FAndroidNativeDownloadWorkerParameterKeys::NOTIFICATION_RESOURCE_SMALL_ICON_NAME = TEXT("NotificationResourceSmallIconName");
const FString FAndroidNativeDownloadWorkerParameterKeys::NOTIFICATION_RESOURCE_SMALL_ICON_TYPE = TEXT("NotificationResourceSmallIconType");
const FString FAndroidNativeDownloadWorkerParameterKeys::NOTIFICATION_RESOURCE_SMALL_ICON_PACKAGE = TEXT("NotificationResourceSmallIconPackage");

FAndroidPlatformBackgroundHttpManager::FJavaClassInfo FAndroidPlatformBackgroundHttpManager::JavaInfo = FAndroidPlatformBackgroundHttpManager::FJavaClassInfo();

static bool bNetworkConnected = false;

static bool GetNetworkConnected()
{
	switch (FPlatformMisc::GetNetworkConnectionType())
	{
		case ENetworkConnectionType::Ethernet:
		case ENetworkConnectionType::Cell:
		case ENetworkConnectionType::WiFi:
		case ENetworkConnectionType::WiMAX:
			return true;
	}
	return false;
}

FAndroidPlatformBackgroundHttpManager::FAndroidPlatformBackgroundHttpManager()
	: bHasPendingCompletes(false)
	, bIsModifyingPauseList(false)
	, bIsModifyingResumeList(false)
	, bIsModifyingCancelList(false)
	, bHasPendingExpectedWorkStart(false)
	, bShouldForceWorkerRequeue(false)
	, bWorkerHadTerminalFinish(false)
	, AndroidBackgroundHTTPManagerDefaultLocalizedText()
{
	if (!OnNetworkConnectionChangedDelegateHandle.IsValid())
	{
		OnNetworkConnectionChangedDelegateHandle = FPlatformMisc::AddNetworkListener(
			FCoreDelegates::FOnNetworkConnectionChanged::FDelegate::CreateRaw(this, &FAndroidPlatformBackgroundHttpManager::OnNetworkConnectionChanged));
	}

	bNetworkConnected = GetNetworkConnected();
}

bool FAndroidPlatformBackgroundHttpManager::HandleRequirementsCheck()
{
	const bool bSupportsJNI = static_cast<bool>(USE_ANDROID_JNI);
	const bool bEnabledInConfigRules = HandleConfigRulesSettings();
	
	const bool bAreRequirementsSupported = bSupportsJNI && bEnabledInConfigRules;
	
	UE_LOGF(LogBackgroundHttpManager, Display, "HandleRequirementsCheck results bAreRequirementsSupported:%d | bSupportsJNI:%d | bEnabledInConfigRules:%d", static_cast<int>(bAreRequirementsSupported), static_cast<int>(bSupportsJNI), static_cast<int>(bEnabledInConfigRules));
	return bAreRequirementsSupported;
}

bool FAndroidPlatformBackgroundHttpManager::HandleConfigRulesSettings()
{
	EAndroidBackgroundDownloadConfigRulesSetting ConfigRuleSetting = GetAndroidBackgroundDownloadConfigRulesSetting();

	UE_LOGF(LogBackgroundHttpManager, Display, "HandleConfigRulesSettings result: %ls", *LexToString(ConfigRuleSetting));

	//If we have no ConfigRulesSetting or the one we have is set to enabled, then we can turn on this feature
	if (ConfigRuleSetting == EAndroidBackgroundDownloadConfigRulesSetting::Disabled)
	{
		//Since this is Disabled we also should explicitly try to cancel any work that may already be scheduled with our WorkID in case it was previously enabled.
		FUEWorkManagerNativeWrapper::CancelBackgroundWork(BackgroundHTTPWorkID);
		return false;
	}

	return true;
}

void FAndroidPlatformBackgroundHttpManager::Initialize(bool bClearPreExistingRequestsAtStartup)
{	
	//Cancel any initial BackgroundHttpWork that is still running as this was kicked off by a previous run of the game
	//and now that we are foregrounded we should wait to do that work until requested again.
	FUEWorkManagerNativeWrapper::CancelBackgroundWork(BackgroundHTTPWorkID);

	// If the boolean is true, wipe the temp files. It may take a while for the Java side to quit, so keep trying until no
	// files are removed. The Java side, if still active, will see the file deletion as a failed attempt, and will keep
	// on trying to download the file, until it's cleaned up.
	if (bClearPreExistingRequestsAtStartup)
	{
		int AttemptsLeft = 4;
		while (AttemptsLeft > 0)
		{
			const int NumDeleted = DeleteAllTemporaryFiles(true, nullptr);
			if (NumDeleted == 0)
				break;
			else
			{
				FPlatformProcess::Sleep(0.25f);
				--AttemptsLeft;
			}
		} 
	}

	//Update cellular preference in case if download was allowed to go over cellular from notification.
	CellularPreference = FPlatformMisc::GetCellularPreference();

	FAndroidBackgroundDownloadDelegates::AndroidBackgroundDownload_OnWorkerStart.BindSP(this, &FAndroidPlatformBackgroundHttpManager::Java_OnWorkerStart);
	FAndroidBackgroundDownloadDelegates::AndroidBackgroundDownload_OnWorkerStop.BindSP(this, &FAndroidPlatformBackgroundHttpManager::Java_OnWorkerStop);
	FAndroidBackgroundDownloadDelegates::AndroidBackgroundDownload_OnProgress.BindSP(this, &FAndroidPlatformBackgroundHttpManager::Java_OnDownloadProgress);
	FAndroidBackgroundDownloadDelegates::AndroidBackgroundDownload_OnComplete.BindSP(this, &FAndroidPlatformBackgroundHttpManager::Java_OnDownloadComplete);
	FAndroidBackgroundDownloadDelegates::AndroidBackgroundDownload_OnMetrics.BindSP(this, &FAndroidPlatformBackgroundHttpManager::Java_OnDownloadMetrics);
	FAndroidBackgroundDownloadDelegates::AndroidBackgroundDownload_OnAllComplete.BindSP(this, &FAndroidPlatformBackgroundHttpManager::Java_OnAllDownloadsComplete);
	FAndroidBackgroundDownloadDelegates::AndroidBackgroundDownload_OnTickWorkerThread.BindSP(this, &FAndroidPlatformBackgroundHttpManager::Java_OnTick);

	AndroidThunkCpp_SetSharedPreferenceString(TEXT("BackgroundPreferences"), TEXT("TempDownloadRootPath"), FBackgroundHttpFileHashHelper().GetTemporaryRootPath());

	AndroidBackgroundHTTPManagerDefaultLocalizedText.InitFromIniSettings("AndroidFetchBackgroundDownload");
	FBackgroundHttpManagerImpl::Initialize(bClearPreExistingRequestsAtStartup);

	// we download to our temporary root path, and then in a subdirectory named after our branch name;
	// this way, the fetch managers can cleanup stuff they don't know (e.g. downloaded temp files from older versions)
	bool bSuccess = false;
	const FString BuildDownloadPath = FPaths::Combine(FBackgroundHttpFileHashHelper::GetTemporaryRootPath(), BuildSettings::GetBranchName());
	if (IAndroidPlatformFile::GetPlatformPhysical().CreateDirectory(*FBackgroundHttpFileHashHelper::GetTemporaryRootPath())) {
		bSuccess = IAndroidPlatformFile::GetPlatformPhysical().CreateDirectory(*BuildDownloadPath);
	}
	if (!bSuccess) {
		UE_LOGF(LogBackgroundHttpManager, Display, "Failed to create build temp download path: %ls", *BuildDownloadPath);
	}
}

void FAndroidPlatformBackgroundHttpManager::Shutdown()
{
	FPlatformMisc::RemoveNetworkListener(OnNetworkConnectionChangedDelegateHandle);
	OnNetworkConnectionChangedDelegateHandle.Reset();

	FBackgroundHttpManagerImpl::Shutdown();
}

bool FAndroidPlatformBackgroundHttpManager::Tick(float DeltaTime)
{
	FBackgroundHttpManagerImpl::Tick(DeltaTime);

	HandleJavaWorkerReconciliation();

	HandlePendingCompletes();
	HandleRequestsWaitingOnJavaThreadsafety();
	UpdateRequestProgress();

	return true;
}

void FAndroidPlatformBackgroundHttpManager::SetCellularPreference(int32 Value)
{
	CellularPreference = Value;
	FPlatformMisc::SetCellularPreference(Value);
}

void FAndroidPlatformBackgroundHttpManager::SetLimitedDataPreference(int32 Value)
{
	// Not implemented yet, as it is not used.
}

void FAndroidPlatformBackgroundHttpManager::ActivatePendingRequests()
{
	UE_LOGF(LogBackgroundHttpManager, VeryVerbose, "ActivatePendingRequests called");

	const bool bForceRequeueFlagged = FPlatformAtomics::InterlockedExchange(&bShouldForceWorkerRequeue, false);
	bool bForceActiveRequestsToRequeue = false;
	if (bForceRequeueFlagged)
	{
		FRWScopeLock ScopeLock(ActiveRequestLock, SLT_ReadOnly);
		{
			int32 ActiveRequestCount = ActiveRequests.Num();
			bForceActiveRequestsToRequeue = (ActiveRequestCount > 0);
			UE_LOGF(LogBackgroundHttpManager, Display, "Found %d active requests to force requeue.", ActiveRequestCount);
		}
	}
	
	bool bHasPendingRequests = false;
	{
		FRWScopeLock ScopeLock(PendingRequestLock, SLT_ReadOnly);
		bHasPendingRequests = (PendingStartRequests.Num() > 0);

		UE_LOGF(LogBackgroundHttpManager, VeryVerbose, "bHasPendingRequests: %d", static_cast<int>(bHasPendingRequests));
	}

	//Queue up all PendingRequests and ActiveRequests with the underlying UEDownloadWorker layer
	//Requires us to have PendingRequests or have flagged ActiveRequests for a re-queue
	if (bHasPendingRequests || bForceActiveRequestsToRequeue)
	{
		UE_CLOGF(bHasPendingRequests, LogBackgroundHttpManager, Display, "Found PendingRequests to activate");
		UE_CLOGF(bForceActiveRequestsToRequeue, LogBackgroundHttpManager, Display, "Forcing Found ActiveRequests to requeue");

		// If the worker is still running, we have to try and stop it.
		if (CVarStopOldJobBeforeStartingNew.GetValueOnAnyThread())
		{
			const auto IsRunningOpt = FUEWorkManagerNativeWrapper::CheckIfBackgroundWorkerIsRunning(BackgroundHTTPWorkID);
			if (!IsRunningOpt.IsSet())
			{
				static bool HasReportedCheckIfBackgroundWorkIsRunningFailing = false;
				if (!HasReportedCheckIfBackgroundWorkIsRunningFailing)
				{
					HasReportedCheckIfBackgroundWorkIsRunningFailing = true;
					UE_LOGF(LogBackgroundHttpManager, Display, "Failed to check if background work ID is already, acting as if it returned false");
				}
			} 
			else if (IsRunningOpt.GetValue())
			{
				if (!JobStopData.IsSet())
				{
					// This is the first time we're noticing this! Tell the manager to quit the job, and remember when we did this.
					UE_LOGF(LogBackgroundHttpManager, Display, "Job with background workID %ls is already running, stopping it now.", *BackgroundHTTPWorkID);
					FUEWorkManagerNativeWrapper::CancelBackgroundWork(BackgroundHTTPWorkID);
					JobStopData = FJobStopData{};
					return;
				}
				else
				{
					constexpr int MaxStopWaitTime = 30;
					const float StoppedSecsAgo = JobStopData->GetWaitTimeInSeconds();
					if (StoppedSecsAgo < MaxStopWaitTime)
					{
						// Let's wait a little longer.
						const int32 StoppedSecsAgoInt = static_cast<int32>(StoppedSecsAgo);
						if (JobStopData->LastReportedSecs != StoppedSecsAgoInt)
						{
							JobStopData->LastReportedSecs = StoppedSecsAgoInt;
							UE_LOGF(LogBackgroundHttpManager, Display, "Job with background workID %ls has been stopping for %d secs now", *BackgroundHTTPWorkID, StoppedSecsAgoInt);
						}
						return;
					}
					else
					{
						// We've waited the maximum amount of seconds. Proceed.
						UE_LOGF(LogBackgroundHttpManager, Warning, "Job with background workID %ls still not stopped after %d secs, continuing", *BackgroundHTTPWorkID, MaxStopWaitTime);
						JobStopData.Reset();
					}
				}
			}
			else if (JobStopData.IsSet())
			{
				// We were waiting before, but now it stopped.
				UE_LOGF(LogBackgroundHttpManager, Display, "Job with background workID %ls stopped in %d secs", *BackgroundHTTPWorkID, static_cast<int32>(JobStopData->GetWaitTimeInSeconds()));
				JobStopData.Reset();
			}
		}

		// UEDownloadWorker.java actively reads and _removes_ this setting, so setting it every time we schedule a request.
		FPlatformMisc::SetCellularPreference(CellularPreference);

		const bool bIsOptional = false;

		JNIEnv* Env = FAndroidApplication::GetJavaEnv();
			
		FAndroidPlatformBackgroundHttpManager::JavaInfo.Initialize();
						
		if (ensureAlwaysMsgf(Env, TEXT("Invalid JavaEnv! Can not activate any pending requests!")))
		{				
			FScopedJavaObject<jobject> DescriptionArray = NewScopedJavaObject(Env, Env->CallStaticObjectMethod(FAndroidPlatformBackgroundHttpManager::JavaInfo.DownloadDescriptionClass, FAndroidPlatformBackgroundHttpManager::JavaInfo.CreateArrayStaticMethod));
			check(DescriptionArray);

			jclass ArrayClass = Env->GetObjectClass(*DescriptionArray);
			CHECK_JNI_RESULT(ArrayClass);

			jmethodID ArrayPutMethod = Env->GetMethodID(ArrayClass, "add", "(Ljava/lang/Object;)Z");
			CHECK_JNI_RESULT(ArrayPutMethod);

			//Build a list of requests we need to translate to JSON and send down to Java for our UEDownloadWorker to process
			TArray<FAndroidBackgroundHttpRequestPtr> RequestsToSendToJavaLayer;
			{
				//All PendingStartRequests should be in the list
				{
					FRWScopeLock ScopeLock(PendingRequestLock, SLT_Write);
					for (FBackgroundHttpRequestPtr& Request : PendingStartRequests)
					{
						FAndroidBackgroundHttpRequestPtr AndroidRequest = StaticCastSharedPtr<FAndroidPlatformBackgroundHttpRequest>(Request);
						if (ensureAlwaysMsgf(AndroidRequest.IsValid(), TEXT("Unexpected illegal non-Android request in PendingStartRequests!")))
						{
							RequestsToSendToJavaLayer.Add(AndroidRequest);
						}
					}

					//Now clear all PendingStartRequests now that they will be processed
					PendingStartRequests.Empty();

					UE_LOGF(LogBackgroundHttpManager, Display, "Adding %d PendingStartRequests to RequestsToSendToJavaLayer", RequestsToSendToJavaLayer.Num());
				}
				

				//Now go through and re-add any existing ActiveRequests to our list
				//This is needed so they are in the new UEDownloadWorker's requests and still get completed
				{
					FRWScopeLock ScopeLock(ActiveRequestLock, SLT_Write);
					
					UE_LOGF(LogBackgroundHttpManager, Display, "Adding %d currently ActiveRequests back to RequestsToSendToJavaLayer", ActiveRequests.Num());

					for (const FBackgroundHttpRequestPtr& Request : ActiveRequests)
					{
						FAndroidBackgroundHttpRequestPtr AndroidRequest = StaticCastSharedPtr<FAndroidPlatformBackgroundHttpRequest>(Request);
						if (ensureAlwaysMsgf(AndroidRequest.IsValid(), TEXT("Unexpected illegal non-Android request in ActiveRequests!")))
						{
							RequestsToSendToJavaLayer.Add(AndroidRequest);
						}
					}

					//Empty active requests and rebuild it with the RequestsToSendToJavaLayer list as that will be the ActiveRequests list once we re-queue
					{
						ActiveRequests.Empty();

						for (FAndroidBackgroundHttpRequestPtr& AndroidRequest : RequestsToSendToJavaLayer)
						{
							ActiveRequests.Add(AndroidRequest);
						}
					}

					//Fix up NumCurrentlyActiveRequests
					NumCurrentlyActiveRequests = ActiveRequests.Num();
					UE_LOGF(LogBackgroundHttpManager, VeryVerbose, "New ActiveRequests list size: %d", NumCurrentlyActiveRequests);
				}
			}

			//Go through and convert these requests to a JSON blob representing a DownloadDescription and attach it to our DownloadDescription list
			UE_LOGF(LogBackgroundHttpManager, Display, "Adding %d requests to UEDownloadableWorker's DownloadDescription list for active work to be done.", RequestsToSendToJavaLayer.Num());
			for (FAndroidBackgroundHttpRequestPtr& Request : RequestsToSendToJavaLayer)
			{
				const FString RequestJSONString = Request->ToJSon();
				FScopedJavaObject<jstring> JavaJSONString = FJavaHelper::ToJavaString(Env, RequestJSONString);
				FScopedJavaObject<jobject> Description = NewScopedJavaObject(Env, Env->CallStaticObjectMethod(FAndroidPlatformBackgroundHttpManager::JavaInfo.DownloadDescriptionClass, FAndroidPlatformBackgroundHttpManager::JavaInfo.CreateDownloadDescriptionFromJsonMethod, *JavaJSONString));
				UE_LOGF(LogBackgroundHttpManager, VeryVerbose, "Generated JSON for request %ls. || %ls", *Request->GetRequestID(), *RequestJSONString);

				bool bAddedRequestAsDescription = FJavaWrapper::CallBooleanMethod(Env, *DescriptionArray, ArrayPutMethod, *Description);
				if (!bAddedRequestAsDescription)
				{
					ensureAlwaysMsgf(false, TEXT("Failed to create and add valid DownloadDescription for request %s"), *Request->GetRequestID());
					//We failed to add this request to our download worker so just mark it as completed so it can send a failure completion
					MarkUnderlyingJavaRequestAsCompleted(Request, false);
				}
			}
				
			//Call JNI function that saves our passed in ArrayList<DownloadDescription> to a file
			const FString FileNameForDownloadDescList = GetFullFileNameForDownloadDescriptionList();
			FScopedJavaObject<jstring> JavaFileNameString = FJavaHelper::ToJavaString(Env, FileNameForDownloadDescList);	
			bool bDidWriteSucceed = Env->CallStaticBooleanMethod(FAndroidPlatformBackgroundHttpManager::JavaInfo.DownloadDescriptionClass, FAndroidPlatformBackgroundHttpManager::JavaInfo.WriteDownloadDescriptionListToFileMethod, *JavaFileNameString, *DescriptionArray);
			// Calling this to signal to the worker that it was spawned by a forgrounded app
			Env->CallStaticVoidMethod(FAndroidPlatformBackgroundHttpManager::JavaInfo.UEDownloadWorkerClass, FAndroidPlatformBackgroundHttpManager::JavaInfo.GameThreadIsActive);	

			bool bDidSuccessfullyScheduleWork = false;

			//If we successfully created the JSON DownloadDescription file, 
			//then we can actually fill out the rest of our worker parameters and schedule the worker to begin downloading
			if (ensureAlwaysMsgf((bDidWriteSucceed), TEXT("Failed to create download description manifest for the WorkManager UEDownloadWorker. Can not schedule download work without the file as nothing will be downloaded!")))
			{
				FUEWorkManagerNativeWrapper::FWorkRequestParametersNative WorkParams;
				WorkParams.WorkerJavaClass = FAndroidPlatformBackgroundHttpManager::JavaInfo.UEDownloadWorkerClass;
				//Setting this to false to avoid notifications clearing when dropping Internet connection.
				WorkParams.bRequireAnyInternet = false;
				WorkParams.bStartAsForegroundService = true;
					
				//Set our DownloadDescription file so that it can be parsed by the worker
				WorkParams.AddDataToWorkerParameters(FAndroidNativeDownloadWorkerParameterKeys::DOWNLOAD_DESCRIPTION_LIST_KEY, FileNameForDownloadDescList);
					
				//Set our MaxActiveDownloads in the underlying java layer to match our expectation
				WorkParams.AddDataToWorkerParameters(FAndroidNativeDownloadWorkerParameterKeys::DOWNLOAD_MAX_CONCURRENT_REQUESTS_KEY, MaxActiveDownloads);

				WorkParams.AddDataToWorkerParameters(FAndroidNativeDownloadWorkerParameterKeys::CELLULAR_HANDLE_KEY, CVarEnableCellularHandling.GetValueOnGameThread());

				//Make sure we pass in localized notification text bits for the important worker keys
				WorkParams.AddDataToWorkerParameters(FAndroidNativeDownloadWorkerParameterKeys::NOTIFICATION_CONTENT_TITLE_KEY, AndroidBackgroundHTTPManagerDefaultLocalizedText.DefaultNotificationText_Title.GetText());
				WorkParams.AddDataToWorkerParameters(FAndroidNativeDownloadWorkerParameterKeys::NOTIFICATION_CONTENT_COMPLETE_TEXT_KEY, AndroidBackgroundHTTPManagerDefaultLocalizedText.DefaultNotificationText_Complete.GetText());
				WorkParams.AddDataToWorkerParameters(FAndroidNativeDownloadWorkerParameterKeys::NOTIFICATION_CONTENT_CANCEL_DOWNLOAD_TEXT_KEY, AndroidBackgroundHTTPManagerDefaultLocalizedText.DefaultNotificationText_Cancel.GetText());
				WorkParams.AddDataToWorkerParameters(FAndroidNativeDownloadWorkerParameterKeys::NOTIFICATION_CONTENT_NO_INTERNET_TEXT_KEY, AndroidBackgroundHTTPManagerDefaultLocalizedText.DefaultNotificationText_NoInternet.GetText());
				WorkParams.AddDataToWorkerParameters(FAndroidNativeDownloadWorkerParameterKeys::NOTIFICATION_CONTENT_AIRPLANE_MODE_TEXT_KEY, AndroidBackgroundHTTPManagerDefaultLocalizedText.DefaultNotificationText_AirplaneMode.GetText());
				WorkParams.AddDataToWorkerParameters(FAndroidNativeDownloadWorkerParameterKeys::NOTIFICATION_CONTENT_DATA_SAVER_ENABLED_TEXT_KEY, AndroidBackgroundHTTPManagerDefaultLocalizedText.DefaultNotificationText_DataSaverEnabled.GetText());
				WorkParams.AddDataToWorkerParameters(FAndroidNativeDownloadWorkerParameterKeys::NOTIFICATION_CONTENT_WAITING_FOR_CELLULAR_TEXT_KEY, AndroidBackgroundHTTPManagerDefaultLocalizedText.DefaultNotificationText_WaitingForCellular.GetText());
				WorkParams.AddDataToWorkerParameters(FAndroidNativeDownloadWorkerParameterKeys::NOTIFICATION_CONTENT_APPROVE_TEXT_KEY, AndroidBackgroundHTTPManagerDefaultLocalizedText.DefaultNotificationText_Approve.GetText());

				//Expect our ContentText to have a {DownloadPercent} argument in it by default, so this will replace that with the Java string format argument so Java can insert the appropriate value
				FFormatNamedArguments Arguments;
				Arguments.Emplace(TEXT("DownloadPercent"), FText::FromString(TEXT("%3d%%")));
				FText UpdatedContentText = FText::Format(AndroidBackgroundHTTPManagerDefaultLocalizedText.DefaultNotificationText_Content.GetText(), Arguments);
				WorkParams.AddDataToWorkerParameters(FAndroidNativeDownloadWorkerParameterKeys::NOTIFICATION_CONTENT_TEXT_KEY, UpdatedContentText);

				UE_LOGF(LogBackgroundHttpManager, Display, "Attempting to schedule UEDownloadableWorker for background work. DownloadDescriptionList FileName:%ls", *FileNameForDownloadDescList);
				bDidSuccessfullyScheduleWork  = FUEWorkManagerNativeWrapper::ScheduleBackgroundWork(BackgroundHTTPWorkID, WorkParams);
			}
			else
			{
				UE_LOGF(LogBackgroundHttpManager, Error, "Failed to write DownloadDescriptionList FileName:%ls", *FileNameForDownloadDescList);
			}

			if (ensureAlwaysMsgf(bDidSuccessfullyScheduleWork, TEXT("Failure to schedule background download worker!")))
			{
				//We have now scheduled some BG work so make sure we know its safe to call delegates
				FPlatformAtomics::InterlockedExchange(&bHasManagerScheduledBGWork, true);

				//Since we just scheduled new work, we can expect to get a call into OnWorkStart or OnWorkStop
				FPlatformAtomics::InterlockedExchange(&bHasPendingExpectedWorkStart, true);
			}
			//We failed to schedule our background worker due to some error, so need to handle error case
			else
			{
				UE_LOGF(LogBackgroundHttpManager, Error, "Failed to schedule background download worker!");

				//Go through and mark all requests as completed so that they can all send failure completions since our worker failed to schedule
				for (FAndroidBackgroundHttpRequestPtr& Request : RequestsToSendToJavaLayer)
				{
					MarkUnderlyingJavaRequestAsCompleted(Request, false);
				}
			}
		}
	}

	UE_LOGF(LogBackgroundHttpManager, VeryVerbose, "ActivatePendingRequests complete");
}

void FAndroidPlatformBackgroundHttpManager::HandleJavaWorkerReconciliation()
{
	//Check if we ended the last worker in a terminal fashion, and thus might need to fix up some underlying work requests
	//if their state doesn't match this terminal state
	const bool bWorkerHadTerminalFinishCopy = FPlatformAtomics::InterlockedExchange(&bWorkerHadTerminalFinish, false);
	const bool bNewWorkerQueued = FPlatformAtomics::AtomicRead(&bHasPendingExpectedWorkStart);
	if (bWorkerHadTerminalFinishCopy && !bNewWorkerQueued)
	{
		//if we are going to force requeue this worker that takes precedent here instead of forcing them to error out
		const bool bShouldForceWorkerRequeueCopy = FPlatformAtomics::AtomicRead(&bShouldForceWorkerRequeue);
		if (!bShouldForceWorkerRequeueCopy)
		{
			UE_LOGF(LogBackgroundHttpManager, Display, "Completing all still active requests with an error as our worker ended with a terminal state while it was still downloading...");
			ForceCompleteAllUnderlyingJavaActiveRequestsWithError();
		}
		else
		{
			UE_LOGF(LogBackgroundHttpManager, Display, "bWorkerHadTerminalFinish set but ignored as Worker is pending forced requeue");
		}
	}
	else if (bWorkerHadTerminalFinishCopy)
	{
		UE_LOGF(LogBackgroundHttpManager, Display, "bWorkerHadTerminalFinish set but ignored as Worker is pending a new worker start");
	}
}

void FAndroidPlatformBackgroundHttpManager::HandlePendingCompletes()
{
	const bool bShouldProcessCompletes = FPlatformAtomics::InterlockedExchange(&bHasPendingCompletes, false);
	if (bShouldProcessCompletes)
	{
		TArray<FAndroidBackgroundHttpRequestPtr> RequestsToComplete;
		//Populate list of requests waiting on completion handling
		{
			FRWScopeLock ScopeLock(ActiveRequestLock, SLT_ReadOnly);

			for (const FBackgroundHttpRequestPtr& Request : ActiveRequests)
			{
				FAndroidBackgroundHttpRequestPtr AndroidRequest = StaticCastSharedPtr<FAndroidPlatformBackgroundHttpRequest>(Request);
				if (HasUnderlyingJavaCompletedRequest(AndroidRequest))
				{
					RequestsToComplete.Add(MoveTemp(AndroidRequest));
				}
			}	
		}

		//Actually complete requests. The CompleteWithExistingResponseData call should end up removing requests from our ActiveRequests list
		for (const FAndroidBackgroundHttpRequestPtr& Request : RequestsToComplete)
		{
			Request->SendDownloadProgressUpdate();

			FString ExistingFilePath;
			int64 ExistingFileSize;
			const bool bDoesCompleteFileExist = CheckForExistingCompletedDownload(Request, ExistingFilePath, ExistingFileSize);

			EHttpResponseCodes::Type ResponseCodeToUse = bDoesCompleteFileExist ? EHttpResponseCodes::Ok : EHttpResponseCodes::Unknown;
			FBackgroundHttpResponsePtr NewResponseWithExistingFile = FPlatformBackgroundHttp::ConstructBackgroundResponse(ResponseCodeToUse, ExistingFilePath);
			Request->CompleteWithExistingResponseData(NewResponseWithExistingFile);
		}
	}
}

void FAndroidPlatformBackgroundHttpManager::UpdateRequestProgress()
{
	FRWScopeLock ScopeLock(ActiveRequestLock, SLT_ReadOnly);
	for (const FBackgroundHttpRequestPtr& Request : ActiveRequests)
	{
		FAndroidBackgroundHttpRequestPtr AndroidRequest = StaticCastSharedPtr<FAndroidPlatformBackgroundHttpRequest>(Request);
		if (ensureAlwaysMsgf(AndroidRequest.IsValid(), TEXT("Invalid request in Active Requests!")))
		{
			AndroidRequest->SendDownloadProgressUpdate();
		}
	}
}

FAndroidBackgroundHttpRequestPtr FAndroidPlatformBackgroundHttpManager::FindRequestByDestinationLocation(const FString& DestinationLocation)
{
	FRWScopeLock ScopeLock(ActiveRequestLock, SLT_ReadOnly);
	for (const FBackgroundHttpRequestPtr& Request : ActiveRequests)
	{
        FAndroidBackgroundHttpRequestPtr AndroidRequest = StaticCastSharedPtr<FAndroidPlatformBackgroundHttpRequest>(Request);
		if (AndroidRequest->DestinationLocation.Equals(DestinationLocation))
		{
			return AndroidRequest;
		}
	}

	return nullptr;
}

void FAndroidPlatformBackgroundHttpManager::PauseRequest(FBackgroundHttpRequestPtr Request)
{
	if (ensureAlwaysMsgf(IsInGameThread(), TEXT("Should only ever call PauseRequest from GameThread! Can not pause!")))
	{
		UE_LOGF(LogBackgroundHttpManager, Verbose, "PauseRequest called on %ls", *Request->GetRequestID());

		//Flag our request as paused
		FAndroidBackgroundHttpRequestPtr AndroidRequest = StaticCastSharedPtr<FAndroidPlatformBackgroundHttpRequest>(Request);
		if (ensureAlwaysMsgf(AndroidRequest.IsValid(), TEXT("Invalid request in Active Requests!")))
		{
			FPlatformAtomics::InterlockedExchange(&(AndroidRequest->bIsPaused), true);
		}

		//Check if the Java list is safe to modify right now, and if so go ahead and add directly to it
		const bool bIsThreadsafeToHandle = !FPlatformAtomics::InterlockedExchange(&bIsModifyingPauseList, true);
		if (bIsThreadsafeToHandle)
		{
			RequestsToPauseByID_Java.Add(AndroidRequest->DestinationLocation);

			const bool bSanityCheck = FPlatformAtomics::InterlockedExchange(&bIsModifyingPauseList, false);
			ensureAlwaysMsgf(bSanityCheck, TEXT("Error in bIsModifyingPauseList! Was false when we expected it to still be true from our lock!"));
		}
		//If its not safe cache off this pause so we can do it the next time it's safe to access the list
		else
		{
			RequestsToPauseByID_GT.Add(AndroidRequest->DestinationLocation);
		}
	}
}

void FAndroidPlatformBackgroundHttpManager::ResumeRequest(FBackgroundHttpRequestPtr Request)
{
	if (ensureAlwaysMsgf(IsInGameThread(), TEXT("Should only ever call ResumeRequest from GameThread! Can not Resume!")))
	{
		UE_LOGF(LogBackgroundHttpManager, Verbose, "ResumeRequest called on %ls", *Request->GetRequestID());

		//Flag our request as resumed
		FAndroidBackgroundHttpRequestPtr AndroidRequest = StaticCastSharedPtr<FAndroidPlatformBackgroundHttpRequest>(Request);
		if (ensureAlwaysMsgf(AndroidRequest.IsValid(), TEXT("Invalid request in Active Requests!")))
		{
			FPlatformAtomics::InterlockedExchange(&(AndroidRequest->bIsPaused), false);
		}

		//Check if the Java list is safe to modify right now, and if so go ahead and add directly to it
		const bool bIsThreadsafeToHandle = !FPlatformAtomics::InterlockedExchange(&bIsModifyingResumeList, true);
		if (bIsThreadsafeToHandle)
		{
			RequestsToResumeByID_Java.Add(AndroidRequest->DestinationLocation);

			const bool bSanityCheck = FPlatformAtomics::InterlockedExchange(&bIsModifyingResumeList, false);
			ensureAlwaysMsgf(bSanityCheck, TEXT("Error in bIsModifyingResumeList! Was false when we expected it to still be true from our lock!"));
		}
		//If its not safe cache off this pause so we can do it the next time it's safe to access the list
		else
		{
			RequestsToResumeByID_GT.Add(AndroidRequest->DestinationLocation);
		}
	}
}

void FAndroidPlatformBackgroundHttpManager::CancelRequest(FBackgroundHttpRequestPtr Request)
{
	if (ensureAlwaysMsgf(IsInGameThread(), TEXT("Should only ever call CancelRequest from GameThread! Can not Cancel!")))
	{
		UE_LOGF(LogBackgroundHttpManager, Verbose, "CancelRequest called on %ls", *Request->GetRequestID());

        //Go ahead and remove our request since its cancelled
        FAndroidBackgroundHttpRequestPtr AndroidRequest = StaticCastSharedPtr<FAndroidPlatformBackgroundHttpRequest>(Request);
        RemoveRequest(AndroidRequest);

		//Check if the Java list is safe to modify right now, and if so go ahead and add directly to it
		const bool bIsThreadsafeToHandle = !FPlatformAtomics::InterlockedExchange(&bIsModifyingCancelList, true);
		if (bIsThreadsafeToHandle)
		{
			RequestsToCancelByID_Java.Add(AndroidRequest->DestinationLocation);

			const bool bSanityCheck = FPlatformAtomics::InterlockedExchange(&bIsModifyingCancelList, false);
			ensureAlwaysMsgf(bSanityCheck, TEXT("Error in bIsModifyingCancelList! Was false when we expected it to still be true from our lock!"));
		}
		//If its not safe cache off this pause so we can do it the next time it's safe to access the list
		else
		{
			RequestsToCancelByID_GT.Add(AndroidRequest->DestinationLocation);
		}
	}
}

void FAndroidPlatformBackgroundHttpManager::HandleRequestsWaitingOnJavaThreadsafety()
{
	//Check for pause requests
	if (RequestsToPauseByID_GT.Num() > 0)
	{
		const bool bIsThreadsafeToHandle = !FPlatformAtomics::InterlockedExchange(&bIsModifyingPauseList, true);
		if (bIsThreadsafeToHandle)
		{
			for (FString& DestinationLocation : RequestsToPauseByID_GT)
			{
				RequestsToPauseByID_Java.Add(MoveTemp(DestinationLocation));
			}

			RequestsToPauseByID_GT.Empty();

			const bool bSanityCheck = FPlatformAtomics::InterlockedExchange(&bIsModifyingPauseList, false);
			ensureAlwaysMsgf(bSanityCheck, TEXT("Error in bIsModifyingPauseList! Was false when we expected it to still be true from our lock!"));
		}
	}

	//Check for resume requests
	if (RequestsToResumeByID_GT.Num() > 0)
	{
		const bool bIsThreadsafeToHandle = !FPlatformAtomics::InterlockedExchange(&bIsModifyingResumeList, true);
		if (bIsThreadsafeToHandle)
		{
			for (FString& DestinationLocation : RequestsToResumeByID_GT)
			{
				RequestsToResumeByID_Java.Add(MoveTemp(DestinationLocation));
			}

			RequestsToResumeByID_GT.Empty();

			const bool bSanityCheck = FPlatformAtomics::InterlockedExchange(&bIsModifyingResumeList, false);
			ensureAlwaysMsgf(bSanityCheck, TEXT("Error in bIsModifyingResumeList! Was false when we expected it to still be true from our lock!"));
		}
	}

	//Check for cancel requests
	if (RequestsToCancelByID_GT.Num() > 0)
	{
		const bool bIsThreadsafeToHandle = !FPlatformAtomics::InterlockedExchange(&bIsModifyingCancelList, true);
		if (bIsThreadsafeToHandle)
		{
			for (FString& DestinationLocation : RequestsToCancelByID_GT)
			{
				RequestsToCancelByID_Java.Add(MoveTemp(DestinationLocation));
			}

			RequestsToCancelByID_GT.Empty();

			const bool bSanityCheck = FPlatformAtomics::InterlockedExchange(&bIsModifyingCancelList, false);
			ensureAlwaysMsgf(bSanityCheck, TEXT("Error in bIsModifyingCancelList! Was false when we expected it to still be true from our lock!"));
		}
	}
}

const FString FAndroidPlatformBackgroundHttpManager::GetFullFileNameForDownloadDescriptionList() const
{
	static const FString BASE_FILE_NAME = TEXT("DownloadDescriptionListJSON");
	static const FString FILE_EXTENSION = TEXT(".ini");

	static const FString RootBGTempDir = GetFileHashHelper()->GetTemporaryRootPath();
	static const FString BGDownloadDescriptionFolder = FPaths::Combine(RootBGTempDir, TEXT("DownloadDescriptionJSONs"));

	FString FileName = FString::Printf(TEXT("%s%s%s"), *BASE_FILE_NAME, *FGuid::NewGuid().ToString(), *FILE_EXTENSION);
	return FPaths::Combine(BGDownloadDescriptionFolder, FileName);
}

void FAndroidPlatformBackgroundHttpManager::Java_OnDownloadProgress(jobject UnderlyingWorker, FString DestinationLocation, int64_t TotalBytesWritten, int64_t BytesWrittenSinceLastCall)
{
	UE_LOGF(LogBackgroundHttpManager, VeryVerbose, "Download Progress... DestinationLocation:%ls BytesWrittenSinceLastCall:%lld TotalBytesWritten:%lld", *DestinationLocation, BytesWrittenSinceLastCall, TotalBytesWritten);

	FAndroidBackgroundHttpRequestPtr FoundRequest = FindRequestByDestinationLocation(DestinationLocation);
	if (FoundRequest.IsValid())
	{
		FoundRequest->UpdateDownloadProgress(TotalBytesWritten, BytesWrittenSinceLastCall);
	}
}

void FAndroidPlatformBackgroundHttpManager::Java_OnDownloadComplete(jobject UnderlyingWorker, FString DestinationLocation, bool bWasSuccess)
{
	UE_LOGF(LogBackgroundHttpManager, Log, "DownloadComplete... DestinationLocation:%ls bWasSuccess:%d", *DestinationLocation, (int)bWasSuccess);

	//Mark associated request as completed so that we can complete it on our next tick
	FAndroidBackgroundHttpRequestPtr CompletedRequest = FindRequestByDestinationLocation(DestinationLocation);
	if (CompletedRequest.IsValid())
	{
		MarkUnderlyingJavaRequestAsCompleted(CompletedRequest, bWasSuccess);
	}
	else
	{
		UE_LOGF(LogBackgroundHttpManager, Log, "Taking no action as DestinationLocation:%ls did not have a corresponding ActiveRequest", *DestinationLocation);
	}
}

void FAndroidPlatformBackgroundHttpManager::Java_OnDownloadMetrics(jobject UnderlyingWorker, FString DestinationLocation, int64_t TotalBytesDownloaded, int64_t DownloadDuration, int64_t DownloadStartTimeUTC, int64_t DownloadEndTimeUTC)
{
	UE_LOGF(LogBackgroundHttpManager, Log, "DownloadMetrics... DestinationLocation:%ls TotalBytesDownloaded:%ld DownloadDuration:%ld", *DestinationLocation, TotalBytesDownloaded, DownloadDuration);

	FAndroidBackgroundHttpRequestPtr CompletedRequest = FindRequestByDestinationLocation(DestinationLocation);
	if (CompletedRequest.IsValid())
	{
		FBackgroundHttpRequestMetricsExtended MetricsExtended;

		MetricsExtended.TotalBytesDownloaded = TotalBytesDownloaded;
		MetricsExtended.DownloadDuration = (float)((double)DownloadDuration / 1000.0f);
		MetricsExtended.FetchStartTimeUTC = FDateTime::FromUnixTimestampDecimal((double)DownloadStartTimeUTC / 1000.0f);
		MetricsExtended.FetchEndTimeUTC = FDateTime::FromUnixTimestampDecimal((double)DownloadEndTimeUTC / 1000.0f);

		CompletedRequest->NotifyRequestMetricsExtendedAvailable(MetricsExtended);
	}
}

void FAndroidPlatformBackgroundHttpManager::MarkUnderlyingJavaRequestAsCompleted(FBackgroundHttpRequestPtr Request, bool bSuccess /*= true*/)
{
	FAndroidBackgroundHttpRequestPtr AndroidRequest = StaticCastSharedPtr<FAndroidPlatformBackgroundHttpRequest>(Request);
	if (ensureAlwaysMsgf((AndroidRequest.IsValid()), TEXT("Invalid Non-Android request!")))
	{
		return MarkUnderlyingJavaRequestAsCompleted(AndroidRequest, bSuccess);
	}
}

void FAndroidPlatformBackgroundHttpManager::MarkUnderlyingJavaRequestAsCompleted(FAndroidBackgroundHttpRequestPtr Request, bool bSuccess /*= true*/)
{
	if (Request.IsValid())
	{
		//Mark as complete so that on our next Tick we can process this request in particular as completed
		FPlatformAtomics::InterlockedExchange(&(Request->bIsCompleted), true);

		//Mark as pending completes so we know to process completed requests at the manager level during our next tick
		FPlatformAtomics::InterlockedExchange(&bHasPendingCompletes, true);

		//Notify object of our complete so that we can send completion notification when all downloads finish.
		Request->NotifyNotificationObjectOfComplete(bSuccess ? EBackgroundHttpRequestCompletionResult::Success : EBackgroundHttpRequestCompletionResult::GenericError);
	}
}

bool FAndroidPlatformBackgroundHttpManager::HasUnderlyingJavaCompletedRequest(FBackgroundHttpRequestPtr Request)
{
	FAndroidBackgroundHttpRequestPtr AndroidRequest = StaticCastSharedPtr<FAndroidPlatformBackgroundHttpRequest>(Request);
	if (ensureAlwaysMsgf((AndroidRequest.IsValid()), TEXT("Invalid Non-Android request checked for completion! Returning false")))
	{
		return HasUnderlyingJavaCompletedRequest(AndroidRequest);
	}

	return false;
}

bool FAndroidPlatformBackgroundHttpManager::HasUnderlyingJavaCompletedRequest(FAndroidBackgroundHttpRequestPtr Request)
{
	if (ensureAlwaysMsgf(Request.IsValid(), TEXT("Invalid request checked for completion! Returning false")))
	{
		return FPlatformAtomics::AtomicRead(&(Request->bIsCompleted));
	}
	
	return false;
}

void FAndroidPlatformBackgroundHttpManager::Java_OnWorkerStart(FString WorkID, jobject UnderlyingWorker)
{
	const bool bShouldRespondToDelegate = FPlatformAtomics::AtomicRead(&bHasManagerScheduledBGWork);
	if (bShouldRespondToDelegate)
	{
		UE_LOGF(LogBackgroundHttpManager, Log, "OnWorkerStart for %ls", *WorkID);

		//With a new worker started flush this result if it hasn't been processed from the previous worker finishing
		FPlatformAtomics::InterlockedExchange(&bWorkerHadTerminalFinish, false);

		//Remove expected work start as workers have started
		FPlatformAtomics::InterlockedExchange(&bHasPendingExpectedWorkStart, false);

		//Remove forced requeue as another worker started and thus we can discard forcing a new worker start now
		//that we have an active worker again
		FPlatformAtomics::InterlockedExchange(&bShouldForceWorkerRequeue, false);
	}
}

void FAndroidPlatformBackgroundHttpManager::Java_OnWorkerStop(FString WorkID, jobject UnderlyingWorker)
{
	const bool bShouldRespondToDelegate = FPlatformAtomics::AtomicRead(&bHasManagerScheduledBGWork);
	if (bShouldRespondToDelegate)
	{
		UE_LOGF(LogBackgroundHttpManager, Log, "OnWorkerStop for %ls", *WorkID);

		FUEWorkManagerNativeWrapper::EAndroidBackgroundWorkResult WorkResult = FUEWorkManagerNativeWrapper::GetWorkResultOnWorker(UnderlyingWorker);
		if (WorkResult == FUEWorkManagerNativeWrapper::EAndroidBackgroundWorkResult::NotSet)
		{
			//Check if our work is stopping unexpectedly (we expect Stop to be called in the event of us scheduling new work)
			//We know this is unexpected as we didn't set a WorkResult yet to denote the worker's state
			bool bHasPendingStart = FPlatformAtomics::AtomicRead(&bHasPendingExpectedWorkStart);
			if (!bHasPendingStart)
			{
				UE_LOGF(LogBackgroundHttpManager, Warning, "Worker forced to end unexpectidly. Setting bShouldForceWorkerRequeue to force a new worker with all existing and pending work");
				FPlatformAtomics::InterlockedExchange(&bShouldForceWorkerRequeue, true);
			}
		}
		else if (WorkResult != FUEWorkManagerNativeWrapper::EAndroidBackgroundWorkResult::Retry)
		{
			//WorkResult was terminal so we need to make sure we reconcile all work as worker is
			//finished on next GameThread tick
			UE_LOGF(LogBackgroundHttpManager, Log, "Worker Ended with Terminal Result for %ls", *WorkID);
			FPlatformAtomics::InterlockedExchange(&bWorkerHadTerminalFinish, true);
		}
	}
}

void FAndroidPlatformBackgroundHttpManager::Java_OnAllDownloadsComplete(jobject UnderlyingWorker, bool bDidAllRequestsSucceed)
{
	//Only respond to this delegate if it is for work we have scheduled
	const bool bShouldRespondToDelegate = FPlatformAtomics::AtomicRead(&bHasManagerScheduledBGWork);
	if (bShouldRespondToDelegate)
	{
		UE_LOGF(LogBackgroundHttpManager, Log, "OnAllDownloadComplete... bWasSuccess:%d", (int)bDidAllRequestsSucceed);
	
		//Mark the worker as Sucess or Failure now so that the worker terminates work instead of retrying.
		//NOTE: We don't want to auto-retry as we have an FAndroidPlatformBackgroundHttpManager spun up if we get this far
		//thus, we want to intelligently create a new worker that will only attempt to redownload failed downloads appropriately
		//instead of retrying and potentially redownloading everything.
		if (bDidAllRequestsSucceed)
		{
			FUEWorkManagerNativeWrapper::SetWorkResultOnWorker(UnderlyingWorker, FUEWorkManagerNativeWrapper::EAndroidBackgroundWorkResult::Success);
		}
		else
		{
			FUEWorkManagerNativeWrapper::SetWorkResultOnWorker(UnderlyingWorker, FUEWorkManagerNativeWrapper::EAndroidBackgroundWorkResult::Failure);
		}

		//Our underlying worker has finished all work, so this should be the same as a terminal finish
		FPlatformAtomics::InterlockedExchange(&bWorkerHadTerminalFinish, true);
	}
}

//Allows us to do things while the Worker is running on that worker's thread since our normal GameThread may be suspended.
//Need to be VERY careful about thread safety here!
void FAndroidPlatformBackgroundHttpManager::Java_OnTick(JNIEnv* Env, jobject UnderlyingWorker)
{
	//Only respond to this delegate if it is for work we have scheduled, otherwise we can end up processing requests early and losing that
	//we needed to pause/cancel/resume them once the new job started
	const bool bShouldRespondToDelegate = FPlatformAtomics::AtomicRead(&bHasManagerScheduledBGWork);
	if (bShouldRespondToDelegate)
	{
		//None of our CHECK_JNI_RESULT macros should think their results are optional
		const bool bIsOptional = false;

		if (ensureAlwaysMsgf((nullptr != Env), TEXT("Invalid JNIEnv for Java_OnTick!")))
		{
			check(UnderlyingWorker);

			//Get worker class so we can query for java methods we need to call
			jclass UnderlyingWorkerClass = Env->GetObjectClass(UnderlyingWorker);
			CHECK_JNI_RESULT(UnderlyingWorkerClass);

			//Handle Pause Requests
			{
				const bool bIsThreadsafeToHandle = !FPlatformAtomics::InterlockedExchange(&bIsModifyingPauseList, true);
				if (bIsThreadsafeToHandle)
				{
					//GetPauseMethod from UnderlyingWorker
					jmethodID PauseMethod = Env->GetMethodID(UnderlyingWorkerClass, "PauseRequest", "(Ljava/lang/String;)V");
					CHECK_JNI_RESULT(PauseMethod);

					for (const FString& DestinationLocation : RequestsToPauseByID_Java)
					{
						FJavaWrapper::CallVoidMethod(Env, UnderlyingWorker, PauseMethod, *FJavaHelper::ToJavaString(Env, DestinationLocation));
					}

					RequestsToPauseByID_Java.Empty();

					const bool bSanityCheck = FPlatformAtomics::InterlockedExchange(&bIsModifyingPauseList, false);
					ensureAlwaysMsgf(bSanityCheck, TEXT("Error in bIsModifyingPauseList! Was false when we expected it to still be true from our lock!"));
				}
			}

			//Handle resume requests
			{
				const bool bIsThreadsafeToHandle = !FPlatformAtomics::InterlockedExchange(&bIsModifyingResumeList, true);
				if (bIsThreadsafeToHandle)
				{
					//GetPauseMethod from UnderlyingWorker
					jmethodID ResumeMethod = Env->GetMethodID(UnderlyingWorkerClass, "ResumeRequest", "(Ljava/lang/String;)V");
					CHECK_JNI_RESULT(ResumeMethod);

					for (const FString& DestinationLocation : RequestsToResumeByID_Java)
					{
						FJavaWrapper::CallVoidMethod(Env, UnderlyingWorker, ResumeMethod, *FJavaHelper::ToJavaString(Env, DestinationLocation));
					}

					RequestsToResumeByID_Java.Empty();

					const bool bSanityCheck = FPlatformAtomics::InterlockedExchange(&bIsModifyingResumeList, false);
					ensureAlwaysMsgf(bSanityCheck, TEXT("Error in bIsModifyingResumeList! Was false when we expected it to still be true from our lock!"));
				}
			}

			//Handle cancel requests
			{
				const bool bIsThreadsafeToHandle = !FPlatformAtomics::InterlockedExchange(&bIsModifyingCancelList, true);
				if (bIsThreadsafeToHandle)
				{
					jmethodID CancelMethod = Env->GetMethodID(UnderlyingWorkerClass, "CancelRequest", "(Ljava/lang/String;)V");
					CHECK_JNI_RESULT(CancelMethod);

					for (const FString& DestinationLocation : RequestsToCancelByID_Java)
					{
						FJavaWrapper::CallVoidMethod(Env, UnderlyingWorker, CancelMethod, *FJavaHelper::ToJavaString(Env, DestinationLocation));
					}

					RequestsToCancelByID_Java.Empty();

					const bool bSanityCheck = FPlatformAtomics::InterlockedExchange(&bIsModifyingCancelList, false);
					ensureAlwaysMsgf(bSanityCheck, TEXT("Error in bIsModifyingCancelList! Was false when we expected it to still be true from our lock!"));
				}
			}
		}
	}
}

void FAndroidPlatformBackgroundHttpManager::ForceCompleteAllUnderlyingJavaActiveRequestsWithError()
{
	FRWScopeLock ScopeLock(ActiveRequestLock, SLT_ReadOnly);
	for (const FBackgroundHttpRequestPtr& Request : ActiveRequests)
	{
		if (!HasUnderlyingJavaCompletedRequest(Request))
		{
			ensureAlwaysMsgf(false, TEXT("Request Still Active during forced error complete: %s"), *(Request->GetRequestID()));

			//mark complete with an error
			MarkUnderlyingJavaRequestAsCompleted(Request, false);
		}
	}
}

void FAndroidPlatformBackgroundHttpManager::FJavaClassInfo::Initialize()
{
	//Should only be populating these with GameThread values! JavaENV is thread-specific so this is very important that all useages of these classes come from the same game thread!
	ensureAlwaysMsgf(IsInGameThread(), TEXT("Called from un-expected thread! Potential error in an implementation of background downloads!"));

	if (!bHasInitialized)
	{
		JNIEnv* Env = FAndroidApplication::GetJavaEnv();
		if (ensureAlwaysMsgf(Env, TEXT("Invalid JNIEnv! Skipping Initialize!")))
		{
			bHasInitialized = true;

			const bool bIsOptional = false;

			//Find jclass information
			{
				UEDownloadWorkerClass = UE::Jni::Class<UE::Jni::Download::FUEDownloadWorker>;
				check(UEDownloadWorkerClass);

				DownloadDescriptionClass = FAndroidApplication::FindJavaClassGlobalRef("com/epicgames/unreal/download/datastructs/DownloadDescription");
				check(DownloadDescriptionClass);
			}

			//find jmethodID for checking if the worker was created using the Android WorkManager
			{
				GameThreadIsActive = FJavaWrapper::FindStaticMethod(Env, UEDownloadWorkerClass, "AndroidThunkJava_GameThreadIsActive",  "()V", bIsOptional);
			}

			//find jmethodID for DownloadDescription methods
			{
				//Grab a bunch of necessary JNI methods we will need to create our DownloadDescriptions
				CreateArrayStaticMethod = FJavaWrapper::FindStaticMethod(Env, DownloadDescriptionClass, "BuildDescriptionArray", "()Ljava/util/ArrayList;", bIsOptional);
				
				WriteDownloadDescriptionListToFileMethod = FJavaWrapper::FindStaticMethod(Env, DownloadDescriptionClass, "WriteDownloadDescriptionListToFile", "(Ljava/lang/String;Ljava/util/ArrayList;)Z", bIsOptional);

				CreateDownloadDescriptionFromJsonMethod = FJavaWrapper::FindStaticMethod(Env, DownloadDescriptionClass, "FromJSON", "(Ljava/lang/String;)Lcom/epicgames/unreal/download/datastructs/DownloadDescription;", bIsOptional);
			}
		}
	}
}

FAndroidPlatformBackgroundHttpManager::EAndroidBackgroundDownloadConfigRulesSetting FAndroidPlatformBackgroundHttpManager::GetAndroidBackgroundDownloadConfigRulesSetting()
{
	const FString* SettingStringValue = FPlatformMisc::GetConfigRulesVariable(AndroidBackgroundDownloadConfigRulesSettingKey);
	if (nullptr == SettingStringValue)
	{
		return EAndroidBackgroundDownloadConfigRulesSetting::Unknown;
	}

	return LexParseString(SettingStringValue);
}

const FString FAndroidPlatformBackgroundHttpManager::LexToString(EAndroidBackgroundDownloadConfigRulesSetting Setting)
{
	switch (Setting)
	{
	case EAndroidBackgroundDownloadConfigRulesSetting::Unknown:
		return TEXT("Unknown");
	case EAndroidBackgroundDownloadConfigRulesSetting::Enabled:
		return TEXT("Enabled");
	case EAndroidBackgroundDownloadConfigRulesSetting::Disabled:
		return TEXT("Disabled");
	case EAndroidBackgroundDownloadConfigRulesSetting::Count:
		return TEXT("Count");
	}

	static_assert(int(EAndroidBackgroundDownloadConfigRulesSetting::Count) == 3, "Added value to EAndroidBackgroundDownloadConfigRulesSetting without updating LexToString");

	ensureAlwaysMsgf(false, TEXT("Missing implementation in EAndroidBackgroundDownloadConfigRulesSetting LexToString!"));
	return TEXT("Unknown");
}

FAndroidPlatformBackgroundHttpManager::EAndroidBackgroundDownloadConfigRulesSetting FAndroidPlatformBackgroundHttpManager::LexParseString(const FString* Setting)
{
	if ((nullptr == Setting) || (Setting->IsEmpty()) || (Setting->Equals(LexToString(EAndroidBackgroundDownloadConfigRulesSetting::Unknown), ESearchCase::IgnoreCase)))
	{
		return EAndroidBackgroundDownloadConfigRulesSetting::Unknown;
	}
	else if (Setting->Equals(LexToString(EAndroidBackgroundDownloadConfigRulesSetting::Enabled), ESearchCase::IgnoreCase))
	{
		return EAndroidBackgroundDownloadConfigRulesSetting::Enabled;
	}
	else if (Setting->Equals(LexToString(EAndroidBackgroundDownloadConfigRulesSetting::Disabled), ESearchCase::IgnoreCase))
	{
		return EAndroidBackgroundDownloadConfigRulesSetting::Disabled;
	}
	else if (Setting->Equals(LexToString(EAndroidBackgroundDownloadConfigRulesSetting::Count), ESearchCase::IgnoreCase))
	{
		return EAndroidBackgroundDownloadConfigRulesSetting::Count;
	}

	static_assert(int(EAndroidBackgroundDownloadConfigRulesSetting::Count) == 3, "Added value to EAndroidBackgroundDownloadConfigRulesSetting without updating LexParseString");

	ensureAlwaysMsgf(false, TEXT("Missing implementation in EAndroidBackgroundDownloadConfigRulesSetting LexParseString!"));
	return EAndroidBackgroundDownloadConfigRulesSetting::Unknown;
}

FAndroidPlatformBackgroundHttpManager::FAndroidBackgroundHTTPManagerDefaultLocalizedText::FAndroidBackgroundHTTPManagerDefaultLocalizedText::FAndroidBackgroundHTTPManagerDefaultLocalizedText()
	: DefaultNotificationText_Title()
	, DefaultNotificationText_Content()
	, DefaultNotificationText_Complete()
	, DefaultNotificationText_Cancel()
	, DefaultNotificationText_NoInternet()
	, DefaultNotificationText_AirplaneMode()
	, DefaultNotificationText_DataSaverEnabled()
	, DefaultNotificationText_WaitingForCellular()
	, DefaultNotificationText_Approve()
{
}

void FAndroidPlatformBackgroundHttpManager::FAndroidBackgroundHTTPManagerDefaultLocalizedText::InitFromIniSettings(const FString& ConfigFileName)
{
	if (ensureAlwaysMsgf(GConfig, TEXT("Invalid GConfig, can not load default localization strings for this manager!")))
	{
		FConfigFile* Config = GConfig->FindConfigFileWithBaseName(*ConfigFileName);
		if (ensureAlwaysMsgf(Config, TEXT("Unable to find .ini file for %s. Can not load localized text for this manager!"), *ConfigFileName))
		{
			const FString TextConfigSection = TEXT("AndroidBackgroundHTTP.DefaultTextLoc");

			//DefaultNotificationText_Title
			{				
				TArray<FString> TitleTextStrings;
				Config->GetArray(*TextConfigSection, TEXT("DefaultNotificationText_Title"), TitleTextStrings);
				ParsePolyglotTextItem(DefaultNotificationText_Title, TEXT("Notification.Title"),TitleTextStrings);
			}

			//DefaultNotificationText_Content
			{
				TArray<FString> ContentTextStrings;
				Config->GetArray(*TextConfigSection, TEXT("DefaultNotificationText_Content"), ContentTextStrings);
				ParsePolyglotTextItem(DefaultNotificationText_Content, TEXT("Notification.ContentText"), ContentTextStrings);
			}

			//DefaultNotificationText_Complete
			{
				TArray<FString> CompleteTextStrings;
				Config->GetArray(*TextConfigSection, TEXT("DefaultNotificationText_Complete"), CompleteTextStrings);
				ParsePolyglotTextItem(DefaultNotificationText_Complete, TEXT("Notification.CompletedContentText"), CompleteTextStrings);
			}

			//DefaultNotificationText_Cancel
			{
				TArray<FString> CancelTextStrings;
				Config->GetArray(*TextConfigSection, TEXT("DefaultNotificationText_Cancel"), CancelTextStrings);
				ParsePolyglotTextItem(DefaultNotificationText_Cancel, TEXT("Notification.CancelText"), CancelTextStrings);
			}

			//DefaultNotificationText_NoInternet
			{
				TArray<FString> NoInternetTextStrings;
				Config->GetArray(*TextConfigSection, TEXT("DefaultNotificationText_NoInternet"), NoInternetTextStrings);
				ParsePolyglotTextItem(DefaultNotificationText_NoInternet, TEXT("Notification.NoInternetText"), NoInternetTextStrings);
			}

			//DefaultNotificationText_AirplaneMode
			{
				TArray<FString> AirplaneModeTextStrings;
				Config->GetArray(*TextConfigSection, TEXT("DefaultNotificationText_AirplaneMode"), AirplaneModeTextStrings);
				ParsePolyglotTextItem(DefaultNotificationText_AirplaneMode, TEXT("Notification.AirplaneModeText"), AirplaneModeTextStrings);
			}

			//DefaultNotificationText_DataSaverEnabled
			{
				TArray<FString> DataSaverEnabledTextStrings;
				Config->GetArray(*TextConfigSection, TEXT("DefaultNotificationText_DataSaverEnabled"), DataSaverEnabledTextStrings);
				ParsePolyglotTextItem(DefaultNotificationText_DataSaverEnabled, TEXT("Notification.DataSaverEnabledText"), DataSaverEnabledTextStrings);
			}

			//DefaultNotificationText_WaitingForCellular
			{
				TArray<FString> WaitingForCellularTextStrings;
				Config->GetArray(*TextConfigSection, TEXT("DefaultNotificationText_WaitingForCellular"), WaitingForCellularTextStrings);
				ParsePolyglotTextItem(DefaultNotificationText_WaitingForCellular, TEXT("Notification.WaitingForCellularText"), WaitingForCellularTextStrings);
			}

			//DefaultNotificationText_Approve
			{
				TArray<FString> ApproveTextStrings;
				Config->GetArray(*TextConfigSection, TEXT("DefaultNotificationText_Approve"), ApproveTextStrings);
				ParsePolyglotTextItem(DefaultNotificationText_Approve, TEXT("Notification.ApproveText"), ApproveTextStrings);
			}

		}
	}

	//Even if the above ends up in parse errors we want to ensure here, but also generate a valid Polyglot text so we can bubble up the error into the notification without crashing
	ForceValidPolyglotText(DefaultNotificationText_Title, TEXT("DefaultNotificationText_Title"));
	ForceValidPolyglotText(DefaultNotificationText_Content, TEXT("DefaultNotificationText_Content"));
	ForceValidPolyglotText(DefaultNotificationText_Complete, TEXT("DefaultNotificationText_Complete"));
	ForceValidPolyglotText(DefaultNotificationText_Cancel, TEXT("DefaultNotificationText_Cancel"));
	ForceValidPolyglotText(DefaultNotificationText_NoInternet, TEXT("DefaultNotificationText_NoInternet"));
	ForceValidPolyglotText(DefaultNotificationText_WaitingForCellular, TEXT("DefaultNotificationText_WaitingForCellular"));
	ForceValidPolyglotText(DefaultNotificationText_Approve, TEXT("DefaultNotificationText_Approve"));
}

void FAndroidPlatformBackgroundHttpManager::FAndroidBackgroundHTTPManagerDefaultLocalizedText::ForceValidPolyglotText(FPolyglotTextData& TextDataOut, const FString& DebugPolyglotTextName)
{
	FText FailureReason;
	if (!ensureAlwaysMsgf(TextDataOut.IsValid(&FailureReason), TEXT("Invalid .ini settings for %s! Invalid localization found! FailureReason: %s"), *DebugPolyglotTextName, *(FailureReason.ToString())))
	{
		TextDataOut = FPolyglotTextData(ELocalizedTextSourceCategory::Engine, TEXT("AndroidBackgroundHttpManager"), TEXT("InvalidPolyglotLocString"), TEXT("InvalidParse"), TEXT("en"));
	}
}

void FAndroidPlatformBackgroundHttpManager::FAndroidBackgroundHTTPManagerDefaultLocalizedText::ParsePolyglotTextItem(FPolyglotTextData& TextDataOut, const FString& LocKey, TArray<FString>& TextEntryList)
{
	TextDataOut.ClearLocalizedStrings();

	//set values that don't require parsing
	TextDataOut.SetCategory(ELocalizedTextSourceCategory::Engine);
	TextDataOut.SetIdentity(TEXT("AndroidBackgroundHttpManager"), LocKey);
	TextDataOut.SetNativeCulture(TEXT("en"));

	//NOTE:
	//Assumption is that each entry will be in the form of: ("CultureString","TranslatedString")
	//
	for (FString& Entry : TextEntryList)
	{
		//Remove start/end filler since we are parsing this manually
		Entry.TrimStartAndEndInline();
		Entry.RemoveFromStart("(");
		Entry.RemoveFromEnd(")");


		TArray<FString> TextCompontents;
		Entry.ParseIntoArray(TextCompontents, TEXT(","), true);

		
		if (ensureAlwaysMsgf((TextCompontents.Num() == 2), TEXT("Invalid entry in .ini file for %s! Expected Format is (\"CultureString\",\"TranslatedString\") . Found: %s"), *LocKey, *Entry))
		{
			FString CultureString;
			FString TranslatedString;
			FParse::QuotedString(*TextCompontents[0], CultureString);
			FParse::QuotedString(*TextCompontents[1], TranslatedString);

			const bool bParsedSuccessfully = (!CultureString.IsEmpty() && !TranslatedString.IsEmpty());
			if (ensureAlwaysMsgf(bParsedSuccessfully, TEXT("Invalid entry for %s. Entry: %s"), *LocKey, *Entry))
			{
				if (CultureString.Equals("native"))
				{
					TextDataOut.SetNativeString(TranslatedString);
				}
				else
				{
					TextDataOut.AddLocalizedString(CultureString, TranslatedString);
				}
			}
		}
	}
}

void FAndroidPlatformBackgroundHttpManager::OnNetworkConnectionChanged(ENetworkConnectionType ConnectionType)
{
	bool bNewNetworkConnected = GetNetworkConnected();

	if (bNewNetworkConnected != bNetworkConnected)
	{
		bNetworkConnected = bNewNetworkConnected;

		UE_LOGF(LogBackgroundHttpManager, Display, "Flagging worker for requeue due to network connection re-establishing.");
		//On reconnecting to the network flag our worker to force requeue to make sure our work is continued
		//Don't requeue the worker if there is no Internet.
		if (ConnectionType != ENetworkConnectionType::None)
		{
			FPlatformAtomics::InterlockedExchange(&bShouldForceWorkerRequeue, true);
		}
	}
}

//
//JNI methods coming from UEDownloadWorker
//

void JNICALL UE::Jni::Download::FUEDownloadWorker::nativeAndroidBackgroundDownloadOnWorkerStart(JNIEnv* jenv, jobject thiz, jstring WorkID)
{
	const FString UEWorkID = FJavaHelper::FStringFromParam(jenv, WorkID);

	const bool bIsMatchingWorkID = UEWorkID.Equals(FAndroidPlatformBackgroundHttpManager::BackgroundHTTPWorkID, ESearchCase::IgnoreCase);
	if (ensureAlwaysMsgf(bIsMatchingWorkID, TEXT("Unexpected WorkID %s sent back in OnWorkerStart! Ignoring un-related worker!"), *UEWorkID))
	{
		UE_LOGF(LogBackgroundHttpManager, Display, "Called OnWorkerStart for %ls", *UEWorkID);
		FAndroidBackgroundDownloadDelegates::AndroidBackgroundDownload_OnWorkerStart.ExecuteIfBound(UEWorkID, thiz);
	}
}

void JNICALL UE::Jni::Download::FUEDownloadWorker::nativeAndroidBackgroundDownloadOnWorkerStop(JNIEnv* jenv, jobject thiz, jstring WorkID)
{
	FString UEWorkID = FJavaHelper::FStringFromParam(jenv, WorkID);

	const bool bIsMatchingWorkID = UEWorkID.Equals(FAndroidPlatformBackgroundHttpManager::BackgroundHTTPWorkID, ESearchCase::IgnoreCase);
	if (ensureAlwaysMsgf(bIsMatchingWorkID, TEXT("Unexpected WorkID %s sent back in OnWorkerStop! Ignoring un-related worker!"), *UEWorkID))
	{
		UE_LOGF(LogBackgroundHttpManager, Display, "Called OnWorkerStop for %ls", *UEWorkID);
		FAndroidBackgroundDownloadDelegates::AndroidBackgroundDownload_OnWorkerStop.ExecuteIfBound(UEWorkID, thiz);
	}
}

void JNICALL UE::Jni::Download::FUEDownloadWorker::nativeAndroidBackgroundDownloadOnProgress(JNIEnv* jenv, jobject thiz, jstring TaskID, jlong TotalBytesWritten, jlong BytesWrittenSinceLastCall)
{
	FString DestinationLocation = FJavaHelper::FStringFromParam(jenv, TaskID);
	int64_t ConvertedTotalBytesWritten = static_cast<uint64_t>(TotalBytesWritten);
	int64_t ConvertedBytesWrittenSinceLastCall = static_cast<uint64_t>(BytesWrittenSinceLastCall);

	FAndroidBackgroundDownloadDelegates::AndroidBackgroundDownload_OnProgress.ExecuteIfBound(thiz, MoveTemp(DestinationLocation), ConvertedTotalBytesWritten, ConvertedBytesWrittenSinceLastCall);
}

void JNICALL UE::Jni::Download::FUEDownloadWorker::nativeAndroidBackgroundDownloadOnComplete(JNIEnv* jenv, jobject thiz, jstring TaskID, jboolean bWasSuccess)
{
	FString DestinationLocation = FJavaHelper::FStringFromParam(jenv, TaskID);
	bool ConvertedbWasSuccess = static_cast<bool>(bWasSuccess);

	FAndroidBackgroundDownloadDelegates::AndroidBackgroundDownload_OnComplete.ExecuteIfBound(thiz, MoveTemp(DestinationLocation), ConvertedbWasSuccess);
}

void JNICALL UE::Jni::Download::FUEDownloadWorker::nativeAndroidBackgroundDownloadOnMetrics(JNIEnv* jenv, jobject thiz, jstring TaskID, jlong TotalBytesDownloaded, jlong DownloadDuration, jlong DownloadStartTimeUTC, jlong DownloadEndTimeUTC)
{
	FString DestinationLocation = FJavaHelper::FStringFromParam(jenv, TaskID);
	int64_t ConvertedTotalBytesDownloaded = static_cast<uint64_t>(TotalBytesDownloaded);
	int64_t ConvertedDownloadDuration = static_cast<uint64_t>(DownloadDuration);
	int64_t ConvertedDownloadStartTimeUTC = static_cast<uint64_t>(DownloadStartTimeUTC);
	int64_t ConvertedDownloadEndTimeUTC = static_cast<uint64_t>(DownloadEndTimeUTC);

	FAndroidBackgroundDownloadDelegates::AndroidBackgroundDownload_OnMetrics.ExecuteIfBound(thiz, MoveTemp(DestinationLocation), ConvertedTotalBytesDownloaded, ConvertedDownloadDuration, ConvertedDownloadStartTimeUTC, ConvertedDownloadEndTimeUTC);
}

void JNICALL UE::Jni::Download::FUEDownloadWorker::nativeAndroidBackgroundDownloadOnAllComplete(JNIEnv* jenv, jobject thiz, jboolean bDidAllRequestsSucceed)
{
	bool ConvertedbDidAllRequestsSucceed = static_cast<bool>(bDidAllRequestsSucceed);
	FAndroidBackgroundDownloadDelegates::AndroidBackgroundDownload_OnAllComplete.ExecuteIfBound(thiz, bDidAllRequestsSucceed);
}

void JNICALL UE::Jni::Download::FUEDownloadWorker::nativeAndroidBackgroundDownloadOnTick(JNIEnv* jenv, jobject thiz)
{
	FAndroidBackgroundDownloadDelegates::AndroidBackgroundDownload_OnTickWorkerThread.ExecuteIfBound(jenv, thiz);
}

#undef LOCTEXT_NAMESPACE
