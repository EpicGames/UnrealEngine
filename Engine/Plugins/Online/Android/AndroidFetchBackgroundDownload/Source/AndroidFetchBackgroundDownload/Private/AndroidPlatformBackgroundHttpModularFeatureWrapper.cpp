// Copyright Epic Games, Inc. All Rights Reserved.

#include "AndroidPlatformBackgroundHttpModularFeatureWrapper.h"

#include "AndroidFetchPlatformBackgroundHttp.h"
#include "PlatformWithModularFeature/ModularFeaturePlatformBackgroundHttp.h"

#include "Features/IModularFeatures.h"


void FAndroidPlatformBackgroundHttpModularFeatureWrapper::Initialize()
{
	FAndroidFetchPlatformBackgroundHttp::Initialize();
}

void FAndroidPlatformBackgroundHttpModularFeatureWrapper::Shutdown()
{
	FAndroidFetchPlatformBackgroundHttp::Shutdown();
}

FBackgroundHttpManagerPtr FAndroidPlatformBackgroundHttpModularFeatureWrapper::CreatePlatformBackgroundHttpManager()
{
	return FAndroidFetchPlatformBackgroundHttp::CreatePlatformBackgroundHttpManager();
}

FBackgroundHttpRequestPtr FAndroidPlatformBackgroundHttpModularFeatureWrapper::ConstructBackgroundRequest()
{
	return FAndroidFetchPlatformBackgroundHttp::ConstructBackgroundRequest();
}

FBackgroundHttpResponsePtr FAndroidPlatformBackgroundHttpModularFeatureWrapper::ConstructBackgroundResponse(int32 ResponseCode, const FString& TempFilePath)
{
	return FAndroidFetchPlatformBackgroundHttp::ConstructBackgroundResponse(ResponseCode, TempFilePath);
}

void FAndroidPlatformBackgroundHttpModularFeatureWrapper::RegisterAsModularFeature()
{
	if (AreRequirementsSupported())
	{
		IModularFeatures::Get().RegisterModularFeature(FModularFeaturePlatformBackgroundHttp::GetModularFeatureName(), this);
	}
}

void FAndroidPlatformBackgroundHttpModularFeatureWrapper::UnregisterAsModularFeature()
{
	if (AreRequirementsSupported())
	{
		IModularFeatures::Get().UnregisterModularFeature(FModularFeaturePlatformBackgroundHttp::GetModularFeatureName(), this);
	}
}

FString FAndroidPlatformBackgroundHttpModularFeatureWrapper::GetDebugModuleName() const
{
	return TEXT("FAndroidPlatformBackgroundHttp");
}

bool FAndroidPlatformBackgroundHttpModularFeatureWrapper::AreRequirementsSupported()
{
	return FAndroidFetchPlatformBackgroundHttp::CheckRequirementsSupported();
}
