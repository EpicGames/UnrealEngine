// Copyright Epic Games, Inc. All Rights Reserved.

#include "AndroidPlatformBackgroundHttpRequest.h"
#include "AndroidPlatformBackgroundHttpManager.h"
#include "Interfaces/IBackgroundHttpResponse.h"
#include "BackgroundHttpModule.h"

#include "Serialization/JsonSerializerMacros.h"
#include "Serialization/JsonWriter.h"

typedef  TJsonWriter<TCHAR, TCondensedJsonPrintPolicy<TCHAR>> FAndroidHttpRequestJsonWriter;
typedef  TJsonWriterFactory<TCHAR, TCondensedJsonPrintPolicy<TCHAR>> FAndroidHttpRequestJsonWriterFactory;

const FStringView FAndroidPlatformBackgroundHttpRequest::URLKey = TEXTVIEW("URLs");
const FStringView FAndroidPlatformBackgroundHttpRequest::RequestIDKey = TEXTVIEW("RequestID");
const FStringView FAndroidPlatformBackgroundHttpRequest::DestinationLocationKey = TEXTVIEW("DestLocation");
const FStringView FAndroidPlatformBackgroundHttpRequest::MaxRetryCountKey = TEXTVIEW("MaxRetryCount");
const FStringView FAndroidPlatformBackgroundHttpRequest::IndividualURLRetryCountKey = TEXTVIEW("IndividualURLRetryCount");
const FStringView FAndroidPlatformBackgroundHttpRequest::RequestPriorityKey = TEXTVIEW("RequestPriority");
const FStringView FAndroidPlatformBackgroundHttpRequest::GroupIDKey = TEXTVIEW("GroupId");
const FStringView FAndroidPlatformBackgroundHttpRequest::bHasCompletedKey = TEXTVIEW("bHasCompleted");
const FStringView FAndroidPlatformBackgroundHttpRequest::bIsPausedKey = TEXTVIEW("bIsPaused");
const FStringView FAndroidPlatformBackgroundHttpRequest::TotalBytesNeededKey = TEXTVIEW("TotalBytesNeeded");

FAndroidPlatformBackgroundHttpRequest::FAndroidPlatformBackgroundHttpRequest()
	: bIsPaused(false)
	, bIsCompleted(false)
	, DownloadProgress(0)
	, DownloadProgressSinceLastUpdateSent(0)
{
}

void FAndroidPlatformBackgroundHttpRequest::CancelRequest()
{
	FAndroidPlatformBackgroundHttpManagerPtr AndroidBGManagerPtr = StaticCastSharedPtr<FAndroidPlatformBackgroundHttpManager>(FBackgroundHttpModule::Get().GetBackgroundHttpManager());
	if (ensureAlwaysMsgf(AndroidBGManagerPtr.IsValid(), TEXT("Found a BackgroundHttpManager that wasn't the AndroidBackgroundHttpManager! This is not supported or expected!")))
	{
		AndroidBGManagerPtr->CancelRequest(SharedThis(this));
	}
}

void FAndroidPlatformBackgroundHttpRequest::PauseRequest()
{
	FAndroidPlatformBackgroundHttpManagerPtr AndroidBGManagerPtr = StaticCastSharedPtr<FAndroidPlatformBackgroundHttpManager>(FBackgroundHttpModule::Get().GetBackgroundHttpManager());
	if (ensureAlwaysMsgf(AndroidBGManagerPtr.IsValid(), TEXT("Found a BackgroundHttpManager that wasn't the AndroidBackgroundHttpManager! This is not supported or expected!")))
	{
		AndroidBGManagerPtr->PauseRequest(SharedThis(this));
	}
}

void FAndroidPlatformBackgroundHttpRequest::ResumeRequest()
{
	FAndroidPlatformBackgroundHttpManagerPtr AndroidBGManagerPtr = StaticCastSharedPtr<FAndroidPlatformBackgroundHttpManager>(FBackgroundHttpModule::Get().GetBackgroundHttpManager());
	if (ensureAlwaysMsgf(AndroidBGManagerPtr.IsValid(), TEXT("Found a BackgroundHttpManager that wasn't the AndroidBackgroundHttpManager! This is not supported or expected!")))
	{
		AndroidBGManagerPtr->ResumeRequest(SharedThis(this));
	}
}

FString FAndroidPlatformBackgroundHttpRequest::ToJSon() const
{
    DestinationLocation = GetDestinationLocation();
    
	FString JsonOutput;
	TSharedRef<FAndroidHttpRequestJsonWriter> Writer = FAndroidHttpRequestJsonWriterFactory::Create(&JsonOutput);
	Writer->WriteObjectStart();
	{
		Writer->WriteValue(RequestIDKey, RequestID);
		Writer->WriteValue(RequestPriorityKey, GetPriorityAsAndroidPriority());
		Writer->WriteValue(MaxRetryCountKey, NumberOfTotalRetries);
		Writer->WriteValue(DestinationLocationKey, DestinationLocation);
		
		//Write this bool as either true/false so it can be parsed in JSON as a java Boolean object
		const bool bIsCompletedCopy = FPlatformAtomics::AtomicRead(&bIsCompleted);
		const FString HasCompletedString = bIsCompletedCopy ? TEXT("true") : TEXT("false");
		Writer->WriteValue(bHasCompletedKey, HasCompletedString);

		//Write this bool as either true/false so it can be parsed in JSON as a java Boolean object
		const bool bIsPausedCopy = FPlatformAtomics::AtomicRead(&bIsPaused);
		const FString IsPausedString = bIsPausedCopy ? TEXT("true") : TEXT("false");
		Writer->WriteValue(bIsPausedKey, bIsPausedCopy);

		//TODO: The intent of this key is to allow multiple download notifications to be active at once and this group would mean all notifications with the same key
		//are lumped together. For now everything is just expected to be in the same group, but if we want to support this we can implement a more meaningful groupID here.
		Writer->WriteValue(GroupIDKey, 0);
		
		//TODO: Should pull this from the .ini. See how we handle ApplePlatformBackgroundHttPManager::RetryResumeDataLimitSetting
		static const int DefaultIndividualURLRetryCount = 3;
		Writer->WriteValue(IndividualURLRetryCountKey, DefaultIndividualURLRetryCount);

		Writer->WriteValue(TotalBytesNeededKey, GetExpectedResultSize());

		Writer->WriteArrayStart(URLKey);
		{
			for (const FString& URL : URLList)
			{
				Writer->WriteValue(URL);
			}
		}
		Writer->WriteArrayEnd();
	}
	Writer->WriteObjectEnd();
	Writer->Close();
	
	return MoveTemp(JsonOutput);
}

FString FAndroidPlatformBackgroundHttpRequest::GetDestinationLocation() const
{
	//For Android we just use the first URL as the destination location as we have to supply this information up front instead of at completion
	if (URLList.Num() > 0)
	{
		const FString FirstURL = URLList[0];
		return FBackgroundHttpModule::Get().GetBackgroundHttpManager()->GetTempFileLocationForURL(FirstURL, GetRequestID());
	}

	return TEXT("");
}

int FAndroidPlatformBackgroundHttpRequest::GetPriorityAsAndroidPriority() const
{
	switch (RequestPriority)
	{
		case EBackgroundHTTPPriority::High:
			return 1;

		case EBackgroundHTTPPriority::Low:
			return -1;

		case EBackgroundHTTPPriority::Normal:
			return 0;

		default:
			ensureAlwaysMsgf(false, TEXT("Missing EBackgroundHTTPPriority in GetPriorityAsAndroidPriority!"));
			return 0;
	}
}

void FAndroidPlatformBackgroundHttpRequest::UpdateDownloadProgress(int64_t TotalDownloaded, int64_t DownloadedSinceLastUpdate)
{
	UE_LOGF(LogBackgroundHttpRequest, VeryVerbose, "Request Update Progress -- RequestDebugID:%ls | OldProgress:%lld | NewProgress:%lld | ProgressSinceLastUpdate:%lld", *GetRequestID(), DownloadProgress, TotalDownloaded, DownloadedSinceLastUpdate);

	FPlatformAtomics::AtomicStore(&DownloadProgress, TotalDownloaded);
	FPlatformAtomics::InterlockedAdd(&DownloadProgressSinceLastUpdateSent, DownloadedSinceLastUpdate);
}

void FAndroidPlatformBackgroundHttpRequest::SendDownloadProgressUpdate()
{
	//The download progress delegate should only be firing on the game thread 
	//so that requestors don't have to worry about thread safety unexpectidly
	ensureAlwaysMsgf(IsInGameThread(), TEXT("Called from un-expected thread! Potential error in an implementation of background downloads!"));

	//Reset our DownloadProgressSinceLastUpdateSent to 0 now that we are sending a progress update
	int64 DownloadProgressSinceLastUpdateSentCopy = FPlatformAtomics::InterlockedExchange(&DownloadProgressSinceLastUpdateSent, 0);
	
	//Don't send any updates if we haven't updated anything since we last sent an update
	if (DownloadProgressSinceLastUpdateSentCopy > 0)
	{
		int64 DownloadProgressCopy = FPlatformAtomics::AtomicRead(&DownloadProgress);

		OnProgressUpdated().ExecuteIfBound(SharedThis(this), DownloadProgressCopy, DownloadProgressSinceLastUpdateSentCopy);
	}
}