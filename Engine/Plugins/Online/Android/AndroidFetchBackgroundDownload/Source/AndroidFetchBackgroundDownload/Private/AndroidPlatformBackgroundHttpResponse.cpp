// Copyright Epic Games, Inc. All Rights Reserved.

#include "AndroidPlatformBackgroundHttpResponse.h"

FAndroidPlatformBackgroundHttpResponse::FAndroidPlatformBackgroundHttpResponse(int32 ResponseCodeIn, const FString& TempContentFilePathIn)
{
    TempContentFilePath = TempContentFilePathIn;
    ResponseCode = ResponseCodeIn;
}
