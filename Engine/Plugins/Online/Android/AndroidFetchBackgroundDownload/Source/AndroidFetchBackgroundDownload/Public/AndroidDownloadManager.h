// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

#include "AndroidPlatformBackgroundHttpManager.h"

namespace UE::Jni::Download
{
	struct FDownloadManager;
}

namespace UE::Online::Download::Android
{
	struct FDownloadManager: IBackgroundHttpManager
	{
		struct FRequest: FBackgroundHttpRequestImpl
		{
			virtual bool ProcessRequest() override;
			virtual void CancelRequest() override;
			virtual void PauseRequest() override;
			virtual void ResumeRequest() override;

#if !UE_BUILD_SHIPPING
			virtual void GetDebugText(TArray<FString>& Output) override;
#endif

			void RotateURLList();

		private:
			int GetPriorityAsAndroidPriority() const;

			FString DestinationLocation;

			TWeakPtr<FDownloadManager> Parent;

			bool bIsPaused = false;

			int64 DownloadedBytes = 0;
			double DownloadStartTime = 0.0;
#if !UE_BUILD_SHIPPING
			FString DebugString;
#endif
			int64 StartTimeUTC = 0;

			friend FDownloadManager;
		};

		FDownloadManager();
		
		virtual void Initialize(bool bClearPreExistingRequestsAtStartup) override;
		virtual void Shutdown() override;
		
		virtual void AddRequest(const FBackgroundHttpRequestPtr Request) override;
		virtual void RemoveRequest(const FBackgroundHttpRequestPtr Request) override;
		virtual void RequeueRequest(const FBackgroundHttpRequestPtr Request) override;
		virtual void GetActiveDownloadRequestIDs(TSet<FString>& OutActiveIDs) const override;

		virtual void DeleteAllTemporaryFiles() override;
		virtual bool IsGenericImplementation() const override { return false; }
		
		virtual void SetCellularPreference(int32 Value) override;
		virtual void SetLimitedDataPreference(int32 Value) override;
		
		virtual int GetMaxActiveDownloads() const override { return MaxActiveDownloads; }
		virtual void SetMaxActiveDownloads(int32 MaxActiveDownloads) override;

		virtual FString GetTempFileLocationForURL(const FString& URL, const FString& RequestID) override { return {}; }
		
		virtual void CleanUpDataAfterCompletingRequest(const FBackgroundHttpRequestPtr Request) override;

#if !UE_BUILD_SHIPPING
		void RestartAllDownloads();
#endif

	private:
		virtual bool AssociateWithAnyExistingRequest(const FBackgroundHttpRequestPtr Request) override { return false; }
		
	private:
		void EnqueueRequest(FRequest& Request);
		void CancelRequest(FRequest& Request);
		
		void OnProgress(const FString& DestinationLocation, int64 TotalBytesWritten, int64 BytesWrittenSinceLastCall) const;
		void OnComplete(const FString& DestinationLocation, bool bDidSucceed, int64 TotalBytesDownloaded, int64 DownloadDuration, int64 DownloadStartTimeUTC, int64 DownloadEndTimeUTC);
		void OnStart(const FString& DestinationLocation, const FString& DebugString, int64 StartTimeUTC) const;

	private:
		void DeleteStaleTempFiles() const;
		static void GatherTempFilesOlderThen(TArray<FString>& OutTimedOutTempFilenames,double SecondsToConsiderOld, const TArray<FString>* OptionalFileList = nullptr);
		void GatherTempFilesWithoutURLMappings(TArray<FString>& OutTempFilesMissingURLMappings, TArray<FString>* OptionalFileList = nullptr) const;
		static void GatherAllTempFilenames(TArray<FString>& OutAllTempFilenames, bool bOutputAsFullPaths = false);
		static void ConvertAllTempFilenamesToFullPaths(TArray<FString>& OutFilenamesAsFullPaths, const TArray<FString>& FilenamesToConvertToFullPaths);
		
	private:
		int MaxActiveDownloads;
		
		TMap<FStringView, TSharedPtr<FRequest>> Requests;

		BackgroundHttpFileHashHelperRef FileHashHelper;
		
		FAndroidPlatformBackgroundHttpManager::FAndroidBackgroundHTTPManagerDefaultLocalizedText AndroidBackgroundHTTPManagerDefaultLocalizedText;
		
		friend Jni::Download::FDownloadManager;
	};
}