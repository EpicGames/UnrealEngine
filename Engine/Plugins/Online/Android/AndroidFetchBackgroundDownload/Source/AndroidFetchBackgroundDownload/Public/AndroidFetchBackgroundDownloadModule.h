// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Modules/ModuleManager.h"

#include "AndroidPlatformBackgroundHttpModularFeatureWrapper.h"

DECLARE_LOG_CATEGORY_EXTERN(LogAndroidBackgroundService, Log, All);

/**
 * The public interface to this module.  In most cases, this interface is only public to sibling modules 
 * within this plugin.
 */
class FAndroidFetchBackgroundDownloadModule : public IModuleInterface
{
public:
    /** IModuleInterface implementation */
	virtual void StartupModule() override;
	virtual void ShutdownModule() override;

private:
	TSharedPtr<FAndroidPlatformBackgroundHttpModularFeatureWrapper, ESPMode::ThreadSafe> FeatureWrapper;
};

