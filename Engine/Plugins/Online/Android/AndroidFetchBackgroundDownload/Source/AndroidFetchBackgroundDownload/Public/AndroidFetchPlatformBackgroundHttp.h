// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Android/AndroidPlatform.h"

#include "CoreMinimal.h"

#include "Interfaces/IBackgroundHttpRequest.h"
#include "Interfaces/IBackgroundHttpManager.h"
#include "Interfaces/IBackgroundHttpResponse.h"


/**
 * Android specific BackgroundHttp ModularFeature implementation that makes use of AndroidBackgroundService and the Fetch API to perform background downloading.
 */
class FAndroidFetchPlatformBackgroundHttp
{
public:
	/**
	 * Platform initialization step
	 */
	static void Initialize();

	/**
	 * Platform shutdown step
	 */
	static void Shutdown();

	/**
	 * Creates a platform-specific Background HTTP manager.
	 * Un-implemented platforms should create a FGenericPlatformBackgroundHttpManager
	 */
	static FBackgroundHttpManagerPtr CreatePlatformBackgroundHttpManager();

	/**
	 * Creates a new Background Http request instance for the current platform
	 * that will continue to download when the application is in the background
	 *
	 * @return request object
	 */
	static FBackgroundHttpRequestPtr ConstructBackgroundRequest();

	/**
	 * Creates a new Background Http Response instance for the current platform
	 * This normally is called by the request and associated with itself.
	 *
	 * @return response object
	 */
	static FBackgroundHttpResponsePtr ConstructBackgroundResponse(int32 ResponseCode, const FString& TempFilePath);

	/**
	 * Allows our underlying Android Background HTTP implementation to check for requirements before we utilize this modular feature
	 *
	 * @return true if our requirements check succeeded
	 */
	static bool CheckRequirementsSupported();
};