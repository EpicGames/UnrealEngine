// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "CoreMinimal.h"

#include "BackgroundHttpManagerImpl.h"
#include "Interfaces/IBackgroundHttpRequest.h"

#include "Internationalization/PolyglotTextData.h"

#include "AndroidPlatformBackgroundHttpRequest.h"

#include "Android/AndroidPlatform.h"
#include "Android/AndroidJava.h"
#include "Android/AndroidJavaEnv.h"
#include "Android/AndroidJNI.h"
#include "Android/AndroidApplication.h"

#include "Misc/DateTime.h"

//Forward Declarations
struct FPolyglotTextData;

namespace UE::Online::Download::Android
{
	struct FDownloadManager;
}

/**
 * Manages Background Http request that are currently being processed if we are on an Android Platform
 */
class FAndroidPlatformBackgroundHttpManager
	: public FBackgroundHttpManagerImpl
{
public:
	FAndroidPlatformBackgroundHttpManager();
	virtual ~FAndroidPlatformBackgroundHttpManager() {};
	
	/**
	* FBackgroundHttpManagerImpl overrides
	*/
public:
	virtual void Initialize(bool bClearPreExistingRequestsAtStartup) override;
	virtual void Shutdown() override;
	virtual void SetCellularPreference(int32 Value) override;
	virtual void SetLimitedDataPreference(int32 Value) override;
	virtual bool IsGenericImplementation() const override { return false; }
	bool Tick(float DeltaTime);
	
protected:
	virtual void ActivatePendingRequests() override;

public :
	void PauseRequest(FBackgroundHttpRequestPtr Request);
	void ResumeRequest(FBackgroundHttpRequestPtr Request);
	void CancelRequest(FBackgroundHttpRequestPtr Request);

public:
	static bool HandleRequirementsCheck();
	
	// Returns true if the ConfigRules settings allow us to use this feature or false if it should be disabled based on those settings.
	// Also cleans up work if appropriate based on settings.
	static bool HandleConfigRulesSettings();

	//Used to identify/schedule BackgroundHTTP work through our UEDownloadWorker
	static const FString BackgroundHTTPWorkID;

protected:
	void UpdateRequestProgress();

	FAndroidBackgroundHttpRequestPtr FindRequestByDestinationLocation(const FString& DestinationLocation);
	void HandlePendingCompletes();

	void HandleJavaWorkerReconciliation();

	const FString GetFullFileNameForDownloadDescriptionList() const;

	//Handlers for our download progressing in the underlying java implementation so that we can bubble it up to UE code.
	void Java_OnWorkerStart(FString WorkID, jobject UnderlyingWorker);
	void Java_OnWorkerStop(FString WorkID, jobject UnderlyingWorker);
	void Java_OnDownloadProgress(jobject UnderlyingWorker, FString DestinationLocation, int64_t TotalBytesWritten, int64_t BytesWrittenSinceLastCall);
	void Java_OnDownloadComplete(jobject UnderlyingWorker, FString DestinationLocation, bool bWasSuccess);
	void Java_OnDownloadMetrics(jobject UnderlyingWorker, FString DestinationLocation, int64_t TotalBytesDownloaded, int64_t DownloadDuration, int64_t DownloadStartTimeUTC, int64_t DownloadEndTimeUTC);
	void Java_OnAllDownloadsComplete(jobject UnderlyingWorker, bool bDidAllRequestsSucceed);
	void Java_OnTick(JNIEnv* Env, jobject UnderlyingWorker);

	//Helper function that completes any un-completed requests in the underlying java tracking layer so that they will
	//complete on the next Tick. Mostly used to help when the underlying java error hits a failure. Ensures when any
	//request isn't actually completed when called
	void ForceCompleteAllUnderlyingJavaActiveRequestsWithError();

	//Helper function to determine if a background http request was completed in the underlying layer
	//but is still in our ActiveRequests lists as its pending being a complete being sent on our game thread tick
	bool HasUnderlyingJavaCompletedRequest(FBackgroundHttpRequestPtr Request);
	bool HasUnderlyingJavaCompletedRequest(FAndroidBackgroundHttpRequestPtr Request);

	//Flag our underlying request as completed in a thread-safe way as this needs to be able to called on the background worker thread or game thread
	void MarkUnderlyingJavaRequestAsCompleted(FBackgroundHttpRequestPtr Request, bool bSuccess = true );
	void MarkUnderlyingJavaRequestAsCompleted(FAndroidBackgroundHttpRequestPtr Request, bool bSuccess = true);

	//Array used to store Pause/Resume/Cancel requests in a thread-safe non-locking way. This way we can utilize the _Java lists in our Java_OnTick
	//without worrying about blocking the java thread
	TArray<FString> RequestsToPauseByID_GT;
	TArray<FString> RequestsToPauseByID_Java;
	TArray<FString> RequestsToResumeByID_GT;
	TArray<FString> RequestsToResumeByID_Java;
	TArray<FString> RequestsToCancelByID_GT;
	TArray<FString> RequestsToCancelByID_Java;

	//Used to flag if we have any downloads completed in our ActiveRequests list that are 
	//completed in java but waiting to send their completion handler on the GameThread
	volatile int32 bHasPendingCompletes;
	
	//Used to ensure that we can skip past the Pause/Resume/Cancel sections if one of the threads is using them
	//Don't want to just use RWLocks as we can't block the WorkerThread
	volatile int32 bIsModifyingPauseList;
	volatile int32 bIsModifyingResumeList;
	volatile int32 bIsModifyingCancelList;

	//Tracks if we are expecting a worker to start
	volatile int32 bHasPendingExpectedWorkStart;

	//If for some reason we should force requeuing all active work even if there are no current PendingDownloadRequests
	volatile int32 bShouldForceWorkerRequeue;
	
	//If we need to reconcile the state of our underlying requests as
	//we have terminally finished our work and want to make sure all work is
	//correctly set to finish.
	volatile int32 bWorkerHadTerminalFinish;

	//Rechecks any _GT lists to try and move them to _Java lists if its safe to do so
	void HandleRequestsWaitingOnJavaThreadsafety();

	void OnNetworkConnectionChanged(ENetworkConnectionType ConnectionType);
	FDelegateHandle OnNetworkConnectionChangedDelegateHandle;

	int32 CellularPreference = 0;
protected:
	//Used to determine the state of AndroidBackgroundHTTP in our configrules
	enum class EAndroidBackgroundDownloadConfigRulesSetting : uint8
	{
		Unknown,				//Either the value was not set, or was set to something besides these expected values
		Enabled,				//Enabled in config rules
		Disabled,				//Disabled in our config rules.
		Count					//Count of enum values
	};

	static const FString LexToString(EAndroidBackgroundDownloadConfigRulesSetting Setting);
	static EAndroidBackgroundDownloadConfigRulesSetting LexParseString(const FString* Setting);

	static EAndroidBackgroundDownloadConfigRulesSetting GetAndroidBackgroundDownloadConfigRulesSetting();

	//Key in ConfigRules to use to load the value of the EAndroidBackgroundDownloadConfigRulesSetting
	static const FString AndroidBackgroundDownloadConfigRulesSettingKey;

protected:
	//Since our manager could be loaded before UObjects, we have to manually parse some default PolyglotTextData from special .ini entries
	//This struct hold that data and handles some simple parsing for us
	struct FAndroidBackgroundHTTPManagerDefaultLocalizedText
	{
		FAndroidBackgroundHTTPManagerDefaultLocalizedText();

		FPolyglotTextData DefaultNotificationText_Title;
		FPolyglotTextData DefaultNotificationText_Content;
		FPolyglotTextData DefaultNotificationText_Complete;
		FPolyglotTextData DefaultNotificationText_Cancel;
		FPolyglotTextData DefaultNotificationText_NoInternet;
		FPolyglotTextData DefaultNotificationText_AirplaneMode;
		FPolyglotTextData DefaultNotificationText_DataSaverEnabled;
		FPolyglotTextData DefaultNotificationText_WaitingForCellular;
		FPolyglotTextData DefaultNotificationText_Approve;


		void InitFromIniSettings(const FString& ConfigFileName);

	private:
		//Fills out a FPolyglotTextData with data from the supplied text entry parsed from the config.ini
		void ParsePolyglotTextItem(FPolyglotTextData& TextDataOut, const FString& LocTextKey, TArray<FString>& TextEntryList);
		
		//Helper that Checks for valid Polyglot text, and if it fails sets up a PolyglotText item that will pass IsValid() but represents a failed parse.
		void ForceValidPolyglotText(FPolyglotTextData& TextDataOut, const FString& DebugPolyglotTextName);
	};

	FAndroidBackgroundHTTPManagerDefaultLocalizedText AndroidBackgroundHTTPManagerDefaultLocalizedText;

private:
	//struct holding all our Java class, method, and field information in one location.
	//Must call initialize on this before it is useful. Future calls to Initialize will not recalculate information
	struct FJavaClassInfo
	{
		bool bHasInitialized = false;

		jclass UEDownloadWorkerClass;
		jclass DownloadDescriptionClass;

		jmethodID GameThreadIsActive;
		//Necessary JNI methods we will need to create our DownloadDescriptions
		jmethodID CreateArrayStaticMethod;
		jmethodID WriteDownloadDescriptionListToFileMethod;
		jmethodID CreateDownloadDescriptionFromJsonMethod;

		void Initialize();

		FJavaClassInfo()
			: UEDownloadWorkerClass(0)
			, DownloadDescriptionClass(0)
			, GameThreadIsActive(0)
			, CreateArrayStaticMethod(0)
			, WriteDownloadDescriptionListToFileMethod(0)
		{}
	};

	static FJavaClassInfo JavaInfo;

	struct FJobStopData
	{
		FDateTime StartTime = FDateTime::UtcNow();
		int32 LastReportedSecs = 0;
		double GetWaitTimeInSeconds() const {
			return (FDateTime::UtcNow() - StartTime).GetTotalSeconds();
		}
	};
	TOptional<FJobStopData> JobStopData;

protected:
	//We only want to respond to certain broadcast java-side work if we have scheduled BGWork
	//otherwise it could be stale work from a previous executables run that would look like an error state (Requests we haven't queued yet, etc)
	static volatile int32 bHasManagerScheduledBGWork;
	
	friend UE::Online::Download::Android::FDownloadManager;
};

//WARNING: These values MUST stay in sync with their values in DownloadWorkerParameterKeys.java!
class FAndroidNativeDownloadWorkerParameterKeys
{
public:
	static const FString DOWNLOAD_DESCRIPTION_LIST_KEY;
	static const FString DOWNLOAD_MAX_CONCURRENT_REQUESTS_KEY;

	static const FString NOTIFICATION_CHANNEL_ID_KEY;
	static const FString NOTIFICATION_CHANNEL_NAME_KEY;
	static const FString NOTIFICATION_CHANNEL_IMPORTANCE_KEY;
	
	static const FString NOTIFICATION_ID_KEY;
	static const int NOTIFICATION_DEFAULT_ID_KEY;

	static const FString CELLULAR_HANDLE_KEY;

	static const FString NOTIFICATION_CONTENT_TITLE_KEY;
	static const FString NOTIFICATION_CONTENT_TEXT_KEY;
	static const FString NOTIFICATION_CONTENT_CANCEL_DOWNLOAD_TEXT_KEY;
	static const FString NOTIFICATION_CONTENT_NO_INTERNET_TEXT_KEY;
	static const FString NOTIFICATION_CONTENT_AIRPLANE_MODE_TEXT_KEY;
	static const FString NOTIFICATION_CONTENT_DATA_SAVER_ENABLED_TEXT_KEY;
	static const FString NOTIFICATION_CONTENT_COMPLETE_TEXT_KEY;
	static const FString NOTIFICATION_CONTENT_WAITING_FOR_CELLULAR_TEXT_KEY;
	static const FString NOTIFICATION_CONTENT_APPROVE_TEXT_KEY;

	static const FString NOTIFICATION_RESOURCE_CANCEL_ICON_NAME;
	static const FString NOTIFICATION_RESOURCE_CANCEL_ICON_TYPE;
	static const FString NOTIFICATION_RESOURCE_CANCEL_ICON_PACKAGE;

	static const FString NOTIFICATION_RESOURCE_SMALL_ICON_NAME;
	static const FString NOTIFICATION_RESOURCE_SMALL_ICON_TYPE;
	static const FString NOTIFICATION_RESOURCE_SMALL_ICON_PACKAGE;

};

typedef TSharedPtr<FAndroidPlatformBackgroundHttpManager, ESPMode::ThreadSafe> FAndroidPlatformBackgroundHttpManagerPtr;