// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Android/AndroidPlatform.h"

#include "CoreMinimal.h"

#include "Interfaces/IBackgroundHttpRequest.h"
#include "Interfaces/IBackgroundHttpManager.h"
#include "Interfaces/IBackgroundHttpResponse.h"

#include "Interfaces/IBackgroundHttpModularFeature.h"

/**
 * Re-routes our modular feature calls to our AndroidFetch BackgroundHttp
 */
class FAndroidPlatformBackgroundHttpModularFeatureWrapper : public IBackgroundHttpModularFeature
{
	/**
	 * IBackgroundHttpModularFeature implementation
	 */
public:
	virtual void Initialize() override;
	virtual void Shutdown() override;
	virtual FBackgroundHttpManagerPtr CreatePlatformBackgroundHttpManager() override;
	virtual FBackgroundHttpRequestPtr ConstructBackgroundRequest() override;
	virtual FBackgroundHttpResponsePtr ConstructBackgroundResponse(int32 ResponseCode, const FString& TempFilePath) override;
	virtual FString GetDebugModuleName() const override;

public:
	//Handles registering / unregistering as a modular feature when appropriate
	//If requirements are not supported we will fail to register and thus fallthrough to generic behaviour
	void RegisterAsModularFeature();
	void UnregisterAsModularFeature();

	//If we actually support everything needed to use this feature
	static bool AreRequirementsSupported();
};