// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Interfaces/IBackgroundHttpRequest.h"
#include "Interfaces/IHttpRequest.h"

#include "BackgroundHttpRequestImpl.h"

/**
 * Contains implementation of Android specific background http requests
 */
class BACKGROUNDHTTP_API FAndroidPlatformBackgroundHttpRequest 
	: public FBackgroundHttpRequestImpl
{
public:
	FAndroidPlatformBackgroundHttpRequest();
	virtual ~FAndroidPlatformBackgroundHttpRequest() {}

	virtual void PauseRequest() override;
    virtual void ResumeRequest() override;
	virtual void CancelRequest() override;
    
	FString ToJSon() const;

	FString GetDestinationLocation() const;

	void UpdateDownloadProgress(int64_t TotalDownloaded, int64_t DownloadedSinceLastUpdate);
	void SendDownloadProgressUpdate();

private:
	int GetPriorityAsAndroidPriority() const;

private:
	//NOTE: These must match the keys used in the DownloadDescription.java file in The AndroidBackgoundService module!
	//We use these keys to convert our request into a DownloadDescription on the java side through JSON serialization!
	static const FStringView URLKey;
	static const FStringView RequestIDKey;
	static const FStringView DestinationLocationKey;
	static const FStringView MaxRetryCountKey;
	static const FStringView IndividualURLRetryCountKey;
	static const FStringView RequestPriorityKey;
	static const FStringView GroupIDKey;
	static const FStringView bHasCompletedKey;
	static const FStringView bIsPausedKey;
	static const FStringView TotalBytesNeededKey;

private:
    mutable FString DestinationLocation;
	volatile int32 bIsPaused;
	volatile int32 bIsCompleted;
	volatile int64 DownloadProgress;
	volatile int64 DownloadProgressSinceLastUpdateSent;

	friend class FAndroidPlatformBackgroundHttpManager;
};

typedef TSharedPtr<class FAndroidPlatformBackgroundHttpRequest, ESPMode::ThreadSafe> FAndroidBackgroundHttpRequestPtr;
